{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Legacy Composure",
	"Description" : "Legacy system for real-time compositing. This plugin is no longer developed. Use Composure going forward.",
	"Category" : "Compositing",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name": "Composure",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "ComposureEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "ComposureLayersEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		}
	],
	"Plugins": [
		{
			"Name": "ComposureShared",
			"Enabled": true
		},
		{
			"Name": "CameraCalibrationCore",
			"Enabled": true
		},
		{
			"Name": "BlueprintMaterialTextureNodes",
			"Enabled": true
		},
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "OpenColorIO",
			"Enabled": true
		},
		{
			"Name": "ActorLayerUtilities",
			"Enabled": true
		},
		{
			"Name": "LensComponent",
			"Enabled": true
		}
	]
}