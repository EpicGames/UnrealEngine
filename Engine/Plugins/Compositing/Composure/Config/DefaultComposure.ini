[/Script/ComposureLayersEditor.DefaultComposureEditorSettings]
+FeaturedCompShotClasses="/Script/Composure.CompositingElement"

+FeaturedElementClasses="/Composure/Blueprints/CompositingElements/BP_MediaPlateCompElement.BP_MediaPlateCompElement_C"
+FeaturedElementClasses="/Composure/Blueprints/CompositingElements/BP_CgCaptureCompElement.BP_CgCaptureCompElement_C"
+FeaturedElementClasses="/Composure/Blueprints/CompositingElements/BP_CgMatteCompElement.BP_CgMatteCompElement_C"

+DefaultElementNames=(("BP_MediaPlateCompElement_C", "media_plate"), ("BP_CgCaptureCompElement_C", "cg_element"), ("BP_CgMatteCompElement_C", "matte_element"), ("BP_DepthCompElement_C", "depth_element"), ("BP_RadialCompElement_C", "radial_element"), ("BP_RampCompElement_C", "ramp_element"), ("BP_RotoPaintCompElement_C", "roto_element"), ("BP_TextureCompElement_C", "texture_element"))

[CoreRedirects]
+ClassRedirects=(OldName="ComposureCompShotElement",NewName="/Script/Composure.CompositingElement")
