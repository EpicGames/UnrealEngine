// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositingElement.h"

#include "CameraCalibrationTypes.h"

#include "CompositingCaptureBase.generated.h"

class USceneCaptureComponent2D;
class ULensComponent;
class ULensDistortionModelHandlerBase;


/**
 * Base class for CG Compositing Elements
 */
UCLASS(BlueprintType)
class COMPOSURE_API ACompositingCaptureBase : public ACompositingElement
{
	GENERATED_BODY()

public:
	/** Component used to generate the scene capture for this CG Layer */
	UPROPERTY(VisibleDefaultsOnly, BlueprintReadOnly, Category = "SceneCapture")
	TObjectPtr<USceneCaptureComponent2D> SceneCaptureComponent2D = nullptr;

protected:
	/** Whether to apply distortion as a post-process effect on this CG Layer */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composure|LensDistortion")
	bool bApplyDistortion = false;

	/** A component reference (customized) that allows the user to specify a component that this controller should control */
	UPROPERTY(EditInstanceOnly, Category = "Composure|LensDistortion", meta=(DisplayName="Lens Component", UseComponentPicker, AllowedClasses = "/Script/CameraCalibrationCore.LensComponent", AllowAnyActor))
	FComponentReference LensComponentPicker;

	/** Value used to augment the FOV of the scene capture to produce a CG image with enough data to distort */
	UPROPERTY(BlueprintReadOnly, Category = "Composure|LensDistortion")
	float OverscanFactor = 1.0f;

	/** Focal length of the target camera before any overscan has been applied */
	UPROPERTY(BlueprintReadOnly, Category = "Composure|LensDistortion")
	float OriginalFocalLength = 35.0f;

	/** Cached distortion MID produced by the Lens Distortion Handler, used to clean up the post-process materials in the case that the the MID changes */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> LastDistortionMID = nullptr;

public:
	/** Default constructor */
	ACompositingCaptureBase();

	/** Update the state of the Lens Distortion Data Handler, and updates or removes the Distortion MID from the SceneCaptureComponent's post process materials, depending on whether distortion should be applied*/
	UFUNCTION(BlueprintCallable, Category = "Composure|LensDistortion")
	void UpdateDistortion();

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif	
	//~ End UObject Interface

public:
	/** Sets whether distortion should be applied or not */
	void SetApplyDistortion(bool bInApplyDistortion);

	/** Set the Lens Component this CG layer will use to drive distortion on the scene capture */
	void SetLens(ULensComponent* InLens);
};
