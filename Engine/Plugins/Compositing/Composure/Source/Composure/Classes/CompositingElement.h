// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComposurePipelineBaseActor.h"
#include "EditorSupport/CompImageColorPickerInterface.h"
#include "CoreMinimal.h"
#include "Engine/TextureRenderTarget2D.h" // for ETextureRenderTargetFormat
#include "Camera/CameraActor.h"
#include "CompositingElements/CompositingTextureLookupTable.h"
#include "EditorSupport/CompFreezeFrameController.h"
#include "CompositingElements/CompositingMaterialPass.h"
#include "CompositingElement.generated.h"

class UComposureCompositingTargetComponent;
class UComposurePostProcessingPassProxy;
class UMaterialInstanceDynamic;
class ACameraActor;
class UTexture;
class UTextureRenderTarget2D;
class FCompElementRenderTargetPool;
class UCompositingElementPass;
class UCompositingElementInput;
class UCompositingElementTransform;
class UCompositingElementOutput;
class UAlphaTransformPass;
struct FInheritedTargetPool;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FDynamicOnTransformPassRendered, class ACompositingElement*, CompElement, UTexture*, Texture, FName, PassName);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FDynamicOnFinalPassRendered, class ACompositingElement*, CompElement, UTexture*, Texture);

UENUM()
enum class ESceneCameraLinkType
{
	Inherited,
	Override,
	Unused, // EDITOR-ONLY value, used to clean up the UI and remove needless params from the details UI on elements that don't need a camera
};

UENUM()
enum class EInheritedSourceType
{
	Inherited,
	Override,
};

UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ETargetUsageFlags : uint8
{
	USAGE_None       = 0x00,
	USAGE_Input      = 1<<0,
	USAGE_Transform  = 1<<1,
	USAGE_Output     = 1<<2,
	USAGE_Persistent = 1<<5,

	// If a pass is tagged 'intermediate' it is still available to the pass immediately 
	// after it. So we ping-pong between intermediate tags, clearing the older one.
	USAGE_Intermediate0 = 1<<3 UMETA(Hidden),
	USAGE_Intermediate1 = 1<<4 UMETA(Hidden),
};
ENUM_CLASS_FLAGS(ETargetUsageFlags);

UENUM()
enum class ECompPassConstructionType
{
	Unknown,
	EditorConstructed,
	BlueprintConstructed,
	CodeConstructed,
};

/**
 * 
 */
UCLASS(BlueprintType, meta=(DisplayName="Empty Comp Shot", ShortTooltip="A simple base actor used to composite multiple render layers together."))
class COMPOSURE_API ACompositingElement : public AComposurePipelineBaseActor, public ICompImageColorPickerInterface
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category = "Composure")
	TObjectPtr<UComposureCompositingTargetComponent> CompositingTarget;

	UPROPERTY(BlueprintReadOnly, Category = "Composure")
	TObjectPtr<UComposurePostProcessingPassProxy> PostProcessProxy;

protected:
	/*********************************/
	// Pipeline Passes 
	//   - protected to prevent users from directly modifying these lists (use the accessor functions instead)

	UPROPERTY(EditAnywhere, Instanced, BlueprintReadOnly, BlueprintGetter = GetInputsList, Category = "Composure|Input", meta=(ShowOnlyInnerProperties))
	TArray<TObjectPtr<UCompositingElementInput>> Inputs;

	UPROPERTY(EditAnywhere, Instanced, BlueprintReadOnly, BlueprintGetter = GetTransformsList, Category = "Composure|Transform/Compositing Passes", meta=(DisplayName = "Transform Passes", ShowOnlyInnerProperties, DisplayAfter="Inputs"))
	TArray<TObjectPtr<UCompositingElementTransform>> TransformPasses;

	UPROPERTY(EditAnywhere, Instanced, BlueprintReadOnly, BlueprintGetter = GetOutputsList, Category = "Composure|Output", meta = (ShowOnlyInnerProperties))
	TArray<TObjectPtr<UCompositingElementOutput>> Outputs;

public:

	/*********************************/
	// Inputs

	UPROPERTY(EditAnywhere, Category = "Composure|Input")
	ESceneCameraLinkType CameraSource = ESceneCameraLinkType::Unused;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TLazyObjectPtr<ACameraActor> TargetCameraActor_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY(EditInstanceOnly, Category = "Composure|Input", DisplayName = "Target Camera Actor")
	TWeakObjectPtr<ACameraActor> TargetCameraActorPtr;

	/*********************************/
	// Outputs

	UPROPERTY(EditAnywhere, Category = "Composure|Output")
	EInheritedSourceType ResolutionSource = EInheritedSourceType::Inherited;

protected:
	UPROPERTY(EditAnywhere, BlueprintSetter=SetRenderResolution, BlueprintGetter=GetRenderResolution, Category="Composure|Output")
 	FIntPoint RenderResolution;
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Composure|Output", AdvancedDisplay)
	TEnumAsByte<ETextureRenderTargetFormat> RenderFormat;

	UPROPERTY(EditDefaultsOnly, Category = "Composure|Output", AdvancedDisplay)
	bool bUseSharedTargetPool;

	/** Called when this comp shot element has rendered one of its internal transform passes */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnTransformPassRendered, ACompositingElement*, UTexture*, FName);
	FOnTransformPassRendered OnTransformPassRendered;

	/** Called when this comp shot element has rendered its final output */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnFinalPassRendered, ACompositingElement*, UTexture*);
	FOnFinalPassRendered OnFinalPassRendered;

	/*********************************/
	// Editor Only

private:
	UPROPERTY(/*BlueprintReadWrite, Category = "Composure",*/ meta = (Bitmask, BitmaskEnum = "/Script/Composure.ETargetUsageFlags"))
	int32 FreezeFrameMask = 0x00;

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Composure|Preview")
	EInheritedSourceType PreviewTransformSource = EInheritedSourceType::Inherited;

	UPROPERTY(EditAnywhere, Instanced, Category = "Composure|Preview")
	TObjectPtr<UCompositingElementTransform> PreviewTransform;

	UPROPERTY(EditDefaultsOnly, Category = "Composure|Editor")
	TSubclassOf<UCompositingElementInput> DefaultInputType;

	UPROPERTY(EditDefaultsOnly, Category = "Composure|Editor")
	TSubclassOf<UCompositingElementTransform> DefaultTransformType;

	UPROPERTY(EditDefaultsOnly, Category = "Composure|Editor")
	TSubclassOf<UCompositingElementOutput> DefaultOutputType;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCompElementConstructed, ACompositingElement*);
	FOnCompElementConstructed OnConstructed;

	FCompFreezeFrameController FreezeFrameController;
#endif // WITH_EDITORONLY_DATA

public:
	/**
	 * Rename composure actor's name
	 * @param NewName             New name for current composure element.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Element")
	void SetElementName(const FName NewName);

	bool AttachAsChildLayer(ACompositingElement* Child);
	
	bool DetatchAsChildLayer(ACompositingElement* Child);

	/**
	 * Determines whether current composure element is a child of another composure element or not.
	 * @return bool                Whether current composure actor is a child actor or not.
	 */
	UFUNCTION(BlueprintPure, Category = "Composure|Element")
	bool IsSubElement() const;

	/**
	 * Get the parent composure element of current element.
	 * @return bool                Whether the function successfully finds the parent or not.
	 */
	UFUNCTION(BlueprintPure, Category = "Composure|Element")
	ACompositingElement* GetElementParent() const;

	/**
	 * Get the first level of current element's child composure elements. 
	 * @return TArray<ACompositingElement*>   The array containing all the first level children without any grandchildren.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Element")
	const TArray<ACompositingElement*> GetChildElements() const;

	template<class T>
	T* AddNewPass(FName PassName, ECompPassConstructionType ConstructedBy = ECompPassConstructionType::CodeConstructed)
	{
		return Cast<T>(AddNewPass(PassName, T::StaticClass(), ConstructedBy));
	}

	UCompositingElementPass* AddNewPass(FName PassName, TSubclassOf<UCompositingElementPass> PassType, ECompPassConstructionType ConstructedBy = ECompPassConstructionType::CodeConstructed);

	/**
	 * Remove a pass from inner pass. This function will not deal with public list shown in the editor. Use DeletePass instead.
	 * @param  ElementPass          The pass that is going to be removed. 
	 * @return bool                 True if deletion operation is successful.
	 */
	bool RemovePass(UCompositingElementPass* ElementPass);

	int32 RemovePassesOfType(TSubclassOf<UCompositingElementPass> PassType);

	/**
	 * Return the rendering opacity of current composure actor.
	 * @return float                The rendering opacity of current composure element.
	 */
	UFUNCTION(BlueprintPure, Category = "Composure|Element")
	float GetOpacity() const { return OutputOpacity; }

	/**
	 * Set the rendering opacity of current composure actor.
	 * @param NewOpacity            The new opacity value set to the composure element. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Element")
	void SetOpacity(const float NewOpacity);

	/**
	 * When set to false, all children composure actor and current actor's auto-rendering behavior will be disabled. This option is
	 * available in the detail panel of the composure element actor as well.
	 * @param bAutoRunChildAndSelf    Enable/Disable all the children and itself's rendering. By default, all rendering is enabled.
	 */
	virtual void SetAutoRunChildrenAndSelf(bool bAutoRunChildAndSelf = true) override;

public:
	//~ ICompImageColorPickerInterface API
	//
	// NOTE: as we cannot make BlueprintCallable functions EditorOnly, we've previously flagged these as 
	//       "DevelopmentOnly", and made them non-functional outside of the editor
	//       However, that restriction has now been removed, since the functions do nothing outside the Editor,
	//       so it is ok for them to be included in packaged games.
	//

	/* EDITOR ONLY - Specifies which intermediate target to pick colors from (if left unset, we default to the display image) */
	UFUNCTION(BlueprintCallable, Category = "Composure|Editor")
	void SetEditorColorPickingTarget(UTextureRenderTarget2D* PickingTarget);

	/* EDITOR ONLY - Specifies an intermediate image to display when picking (if left unset, we default to the final output image) */
	UFUNCTION(BlueprintCallable, Category = "Composure|Editor")
	void SetEditorColorPickerDisplayImage(UTexture* PickerDisplayImage);

#if WITH_EDITOR
	virtual void OnBeginPreview() override;
	virtual UTexture* GetEditorPreviewImage() override;
	virtual void OnEndPreview() override;
	virtual bool UseImplicitGammaForPreview() const override;
	virtual UTexture* GetColorPickerDisplayImage() override;
	virtual UTextureRenderTarget2D* GetColorPickerTarget() override;
	virtual FCompFreezeFrameController* GetFreezeFrameController() override;
#endif

public:
	UFUNCTION(BlueprintNativeEvent, Category = "Composure", meta = (CallInEditor = "true"))
	UTexture* RenderCompElement(bool bCameraCutThisFrame);

public:
	//~ Blueprint API

	/** Called when a transform pass on this element is rendered */
	UPROPERTY(BlueprintAssignable, BlueprintCallable, DisplayName=OnTransformPassRendered, Category="Composure")
	FDynamicOnTransformPassRendered OnTransformPassRendered_BP;

	/** Called when the final output of this element is rendered */
	UPROPERTY(BlueprintAssignable, BlueprintCallable, DisplayName=OnFinalPassRendered, Category="Composure")
	FDynamicOnFinalPassRendered OnFinalPassRendered_BP;

	/** Return the FName of the composure element object*/
	UFUNCTION(BlueprintPure, Category = "Composure|Element")
	FName GetCompElementName() const { return CompShotIdName; }

	UFUNCTION(BlueprintCallable, Category = "Composure")
	UTextureRenderTarget2D* RequestNamedRenderTarget(const FName ReferenceName, const float RenderPercentage = 1.f, ETargetUsageFlags UsageTag = ETargetUsageFlags::USAGE_None);

	UFUNCTION(BlueprintCallable, Category = "Composure")
	bool ReleaseOwnedTarget(UTextureRenderTarget2D* OwnedTarget);

	UFUNCTION(BlueprintCallable, Category = "Composure")
	UTexture* RenderCompositingMaterial(UPARAM(ref) FCompositingMaterial& CompMaterial, float RenderScale = 1.f, FName ResultLookupName = NAME_None, ETargetUsageFlags UsageTag = ETargetUsageFlags::USAGE_None);

	UFUNCTION(BlueprintCallable, Category = "Composure")
	UTextureRenderTarget2D* RenderCompositingMaterialToTarget(UPARAM(ref) FCompositingMaterial& CompMaterial, UTextureRenderTarget2D* RenderTarget, FName ResultLookupName = NAME_None);

	UFUNCTION(BlueprintPure, Category = "Composure|Input")
	ACameraActor* FindTargetCamera() const;

	UFUNCTION(BlueprintCallable, Category = "Composure|Input")
	void SetTargetCamera(ACameraActor* NewCameraActor);

	UFUNCTION(BlueprintCallable, Category = "Composure")
	void RegisterPassResult(FName ReferenceName, UTexture* PassResult, bool bSetAsLatestRenderResult = true);
	UFUNCTION(BlueprintCallable, Category = "Composure")
	UTexture* FindNamedRenderResult(FName PassName, bool bSearchSubElements = true); //const;
	UFUNCTION(BlueprintCallable, Category = "Composure")
	UTexture* GetLatestRenderResult() const;

	UFUNCTION(BlueprintGetter)
	FIntPoint GetRenderResolution() const;

	UFUNCTION(BlueprintSetter)
	void SetRenderResolution(FIntPoint NewResolution) { RenderResolution = NewResolution; }

	/*********************************/
	// Pass Management 

	UFUNCTION(BlueprintCallable, Category = "Composure", meta = (DeterminesOutputType = "InputType"))
	UCompositingElementInput* FindInputPass(UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UCompositingElementInput> InputType, UTexture*& PassResult, FName OptionalPassName = NAME_None);
	UFUNCTION(BlueprintCallable, Category = "Composure", meta = (DeterminesOutputType = "TransformType"))
	UCompositingElementTransform* FindTransformPass(UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UCompositingElementTransform> TransformType, UTexture*& PassResult, FName OptionalPassName = NAME_None);
	UFUNCTION(BlueprintCallable, Category = "Composure", meta = (DeterminesOutputType = "OutputType"))
	UCompositingElementOutput* FindOutputPass(UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UCompositingElementOutput> OutputType, FName OptionalPassName = NAME_None);
	
	UFUNCTION(BlueprintGetter)
	TArray<UCompositingElementInput*> GetInputsList() const { return ToRawPtrTArrayUnsafe(GetInternalInputsList()); }
	UFUNCTION(BlueprintGetter)
	TArray<UCompositingElementTransform*> GetTransformsList() const { return ToRawPtrTArrayUnsafe(GetInternalTransformsList()); }
	UFUNCTION(BlueprintGetter)
	TArray<UCompositingElementOutput*> GetOutputsList() const { return ToRawPtrTArrayUnsafe(GetInternalOutputsList()); }

	/**
	 * Delete a specific pass. This function deals with the public list where deletion is directly reflected in the editor.
	 * @param  PassToDelete			  The pass that will be deleted.
	 * @return bool                   Whether the delete operation is successful or not
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Pass")
	bool DeletePass(UCompositingElementPass* PassToDelete);

	/**
	 * Create a new input pass into the public list which directly shows in the editor. 
	 * @param  PassName                       The name for the new pass.
	 * @param  InputType                      The class type of the created pass.
	 * @return CompositingElementInput        The newly created input pass object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Pass", meta = (DeterminesOutputType = "InputType"))
	UCompositingElementInput* CreateNewInputPass(FName PassName, UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UCompositingElementInput> InputType);

	/**
	 * Create a new Transform pass into the public list which directly shows in the editor.
	 * @param  PassName                       The name for the new pass.
	 * @param  TransformType                  The class type of the created pass.
	 * @return CompositingElementTransform    The newly created transform pass object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Pass", meta = (DeterminesOutputType = "TransformType"))
	UCompositingElementTransform* CreateNewTransformPass(FName PassName, UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UCompositingElementTransform> TransformType);

	/**
	 * Create a new Output pass into the public list which directly shows in the editor.
	 * @param  PassName                       The name for the new pass.
	 * @param  OutputType                     The class type of the created pass.
	 * @return CompositingElementOutput       The newly created output pass object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Pass", meta = (DeterminesOutputType = "OutputType"))
	UCompositingElementOutput* CreateNewOutputPass(FName PassName, UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UCompositingElementOutput> OutputType);

protected:
	UFUNCTION(BlueprintCallable, Category = "Composure|Input", meta = (DeterminesOutputType = "InputType", BlueprintProtected))
	UCompositingElementInput* AddNewInputPass(FName PassName, UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UCompositingElementInput> InputType);
	UFUNCTION(BlueprintCallable, Category = "Composure|Input", meta = (DeterminesOutputType = "TransformType", BlueprintProtected))
	UCompositingElementTransform* AddNewTransformPass(FName PassName, UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UCompositingElementTransform> TransformType);
	UFUNCTION(BlueprintCallable, Category = "Composure|Input", meta = (DeterminesOutputType = "OutputType", BlueprintProtected))
	UCompositingElementOutput* AddNewOutputPass(FName PassName, UPARAM(meta = (AllowAbstract = "false"))TSubclassOf<UCompositingElementOutput> OutputType);

public: 
	//~ Begin UObject interface
	virtual void BeginPlay() override;
	virtual void PostInitProperties() override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
	virtual void PostEditUndo() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
#endif
	//~ End UObject interface
	
	//~ Begin AActor interface
#if WITH_EDITOR
	virtual void RerunConstructionScripts() override;
	virtual void OnConstruction(const FTransform& Transform) override;
#endif
	//~ End AAcotr interface

	//~ Begin AComposurePipelineBaseActor interface
	virtual void SetAutoRun(bool bNewAutoRunVal) override;
	
	virtual void EnqueueRendering_Implementation(bool bCameraCutThisFrame) override;
	virtual bool IsActivelyRunning_Implementation() const;

	UFUNCTION(BlueprintPure, Category = "Composure")
	virtual int32 GetRenderPriority() const override;
	//~ End AComposurePipelineBaseActor interface

private:
	void FrameReset();

	void PostSerializeCompatUpgrade(const int32 ComposureVersion);
	void PostLoadCompatUpgrade(const int32 ComposureVersion);

#if WITH_EDITOR
	UCompositingElementTransform* GetPreviewPass() const;
	bool IsPreviewing() const;

	void OnPIEStarted(bool bIsSimulating);
	void SetDebugDisplayImage(UTexture* DebugDisplayImg);
#endif
	void OnDisabled();

	void RefreshAllInternalPassLists();
	void RefreshInternalInputsList();
	void RefreshInternalTransformsList();
	void RefreshInternalOutputsList();

	const decltype(Inputs)& GetInternalInputsList() const;
	const decltype(TransformPasses)& GetInternalTransformsList() const;
	const decltype(Outputs)& GetInternalOutputsList() const;

	void BeginFrameForAllPasses(bool bCameraCutThisFrame);
	void GenerateInputs();
	void ApplyTransforms(FInheritedTargetPool& RenderTargetPool);
	void RelayOutputs(const FInheritedTargetPool& RenderTargetPool);
	void EndFrameForAllPasses();

	void UpdateFinalRenderResult(UTexture* RenderResult);

	typedef TSharedPtr<FCompElementRenderTargetPool> FSharedTargetPoolPtr;
	const FSharedTargetPoolPtr& GetRenderTargetPool();

	void RegisterTaggedPassResult(FName ReferenceName, UTexture* PassResult, ETargetUsageFlags UsageFlags = ETargetUsageFlags::USAGE_None);
	void ResetResultsLookupTable(bool bKeepPassResults = false);

	void IncIntermediateTrackingTag();

private:
	UPROPERTY()
	FName CompShotIdName;

	UPROPERTY()
	TObjectPtr<ACompositingElement> Parent;
	UPROPERTY()
	TArray<TObjectPtr<ACompositingElement>> ChildLayers;

	/** EDITOR ONLY - Properties associated with */
#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UTexture> DisabledMsgImage;
	UPROPERTY(Transient)
	TObjectPtr<UTexture> EmptyWarnImage;
	UPROPERTY(Transient)
	TObjectPtr<UTexture> SuspendedDbgImage;
	UPROPERTY(Transient)
	TObjectPtr<UTexture> CompilerErrImage;

	UPROPERTY(Transient, DuplicateTransient)
	bool bUsingDebugDisplayImage = false;

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UTexture> ColorPickerDisplayImage;
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UTexture> EditorPreviewImage;

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UTextureRenderTarget2D> ColorPickerTarget;

	uint32 LastEnqueuedFrameId = (uint32)-1;
	int32  PreviewCount = 0;
#endif // WITH_EDITORONLY_DATA

	ETargetUsageFlags NextIntermediateTrackingTag = ETargetUsageFlags::USAGE_Intermediate0;

	UPROPERTY()
	float OutputOpacity = 1.f;

	/** 
	 * Lists containing passes added programatically (or through Blueprints) via the AddNewPass() functions. 
	 * These need their own separate lists to: 1) hide from the details panel, and 2) clear on 
	 * re-construction, so we don't perpetually grow the lists.
	 */
	UPROPERTY()
	TMap<TObjectPtr<UCompositingElementInput>, ECompPassConstructionType> UserConstructedInputs;
	UPROPERTY()
	TMap<TObjectPtr<UCompositingElementTransform>, ECompPassConstructionType> UserConstructedTransforms;
	UPROPERTY()
	TMap<TObjectPtr<UCompositingElementOutput>, ECompPassConstructionType> UserConstructedOutputs;

	/** 
	 * Authoritative lists that we use to iterate on the passes - conjoined from the public lists and the  
	 * internal 'UserConstructed' ones. Used to: 1) have a single 'goto' list (w/ no nullptrs), and 2)
	 * determine passes that were cleared from the public lists so we can halt their processing (still 
	 * alive via the transaction buffer).
	 */
	UPROPERTY(Instanced, Transient, DuplicateTransient, SkipSerialization)
	TArray<TObjectPtr<UCompositingElementInput>> InternalInputs;
	UPROPERTY(Instanced, Transient, DuplicateTransient, SkipSerialization)
	TArray<TObjectPtr<UCompositingElementTransform>> InternalTransformPasses;
	UPROPERTY(Instanced, Transient, DuplicateTransient, SkipSerialization)
	TArray<TObjectPtr<UCompositingElementOutput>> InternalOutputs;

	UPROPERTY(Transient)
	TObjectPtr<UAlphaTransformPass> InternalAlphaPass = nullptr;

	/** */
	FCompositingTextureLookupTable PassResultsTable;
	/** */
	FSharedTargetPoolPtr RenderTargetPool;
};

