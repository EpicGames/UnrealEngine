// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositingElements/CompositingElementPasses.h"
#include "CompositingElements/CompositingMaterialPass.h"
#include "UObject/Interface.h"
#include "CompositingElementInputs.generated.h"

/* UCompositingMediaInput
 *****************************************************************************/

class UMaterialInterface;

UCLASS(Abstract)
class COMPOSURE_API UCompositingMediaInput : public UCompositingElementInput
{
	GENERATED_BODY()

public:
	UCompositingMediaInput();

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (DisplayAfter = "MediaSource", EditCondition = "bEnabled"))
	FCompositingMaterial MediaTransformMaterial;

	// @TODO: Replace MediaTransformMaterial with this
// 	UPROPERTY(EditAnywhere, Instanced, Category="Output Target", meta=(DisplayName="Color Conversion", ShowOnlyInnerProperties))
// 	UCompositingElementTransform* ColorConverter;

public:
	//~ Begin UObject interface
	virtual void PostInitProperties() override;
	//~ End UObject interface

	//~ Begin UCompositingElementInput interface	
	virtual UTexture* GenerateInput_Implementation() override;
	//~ End UCompositingElementInput interface	

private:
	virtual UTexture* GetMediaTexture() const PURE_VIRTUAL(UCompositingMediaInput::GetMediaTexture, return nullptr;);

	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> DefaultMaterial;
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> DefaultTestPlateMaterial;
	UPROPERTY(Transient, DuplicateTransient, SkipSerialization)
	TObjectPtr<UMaterialInstanceDynamic> FallbackMID;
};

/* UMediaBundleCompositingInput
 *****************************************************************************/

// class UMediaBundle;
// 
// UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "Media Bundle Input"))
// class COMPOSURE_API UMediaBundleCompositingInput : public UCompositingMediaInput
// {
// 	GENERATED_BODY()
// 
// public:
// 	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
// 	UMediaBundle* MediaSource;
// 
// private:
// 	//~ UCompositingMediaInput interface	
// 	virtual UTexture* GetMediaTexture() const override;
// };

/* UMediaTextureCompositingInput
 *****************************************************************************/

class UMediaTexture;

UCLASS(BlueprintType, Blueprintable)
class COMPOSURE_API UMediaTextureCompositingInput : public UCompositingMediaInput
{
	GENERATED_BODY()

public:
	UMediaTextureCompositingInput();

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	TObjectPtr<UMediaTexture> MediaSource;

private:
	//~ UCompositingMediaInput interface	
	virtual UTexture* GetMediaTexture() const override;
};

/* UCompositingInputInterfaceProxy
 *****************************************************************************/

UINTERFACE(MinimalAPI)
class UCompositingInputInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class ICompositingInputInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, Category = "Composure", meta = (CallInEditor = "true"))
	void OnFrameBegin(class UCompositingInputInterfaceProxy* Proxy, bool bCameraCutThisFrame);

	UFUNCTION(BlueprintNativeEvent, Category = "Composure|Input", meta = (CallInEditor = "true"))
	UTexture* GenerateInput(class UCompositingInputInterfaceProxy* Proxy);

	UFUNCTION(BlueprintNativeEvent, Category = "Composure", meta = (CallInEditor = "true"))
	void OnFrameEnd(class UCompositingInputInterfaceProxy* Proxy);
};

UCLASS(noteditinlinenew, hidedropdown)
class COMPOSURE_API UCompositingInputInterfaceProxy : public UCompositingElementInput
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadWrite, Category = "Compositing Pass")
	TScriptInterface<ICompositingInputInterface> CompositingInput;

public:
	//~ UCompositingElementInput interface
	virtual void OnFrameBegin_Implementation(bool bCameraCutThisFrame) override;
	virtual UTexture* GenerateInput_Implementation() override;
	virtual void OnFrameEnd_Implementation() override;
};