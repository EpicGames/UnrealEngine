// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositingElements/CompositingElementPasses.h"
#include "Engine/EngineTypes.h" // for FDirectoryPath
#include "Misc/FrameRate.h"
#include "EditorSupport/CompEditorImagePreviewInterface.h"
#include "CompositingElementOutputs.generated.h"

/* UColorConverterOutputPass
 *****************************************************************************/

class UMediaOutput;
class UMediaCapture;
class UCompositingElementTransform;
class UTextureRenderTarget2D;

UCLASS(Abstract)
class COMPOSURE_API UColorConverterOutputPass : public UCompositingElementOutput, public ICompEditorImagePreviewInterface
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Instanced, Category="Compositing Pass", meta=(DisplayName="Color Conversion", DisplayAfter = "PassName", EditCondition = "bEnabled", ShowOnlyInnerProperties))
	TObjectPtr<UCompositingElementTransform> ColorConverter;

public:
	//~ Begin UObject interface
	virtual void PostInitProperties() override;
	//~ End UObject interface

	//~ Begin UCompositingElementOutput interface
	virtual void OnFrameBegin_Implementation(bool bCameraCutThisFrame) override;
	virtual void Reset_Implementation() override;
	virtual void OnDisabled_Implementation() override;
	//~ End UCompositingElementOutput interface

	//~ Begin ICompEditorImagePreviewInterface interface
#if WITH_EDITOR
	virtual UTexture* GetEditorPreviewImage() override { return PreviewResult; }
	virtual bool UseImplicitGammaForPreview() const override { return false; }
#endif
	//~ End ICompEditorImagePreviewInterface interface

protected:
	UTexture* ApplyColorTransform(UTexture* RenderResult, UComposurePostProcessingPassProxy* PostProcessProxy, FInheritedTargetPool& TargetPool);
	UTexture* ApplyColorTransform(UTexture* RenderResult, UComposurePostProcessingPassProxy* PostProcessProxy, FIntPoint TargetResolution, ETextureRenderTargetFormat TargetFormat);
	UTexture* ApplyColorTransform(UTexture* RenderResult, UComposurePostProcessingPassProxy* PostProcessProxy);

	UPROPERTY(Transient)
	TSubclassOf<UCompositingElementTransform> DefaultConverterClass;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UTexture> PreviewResult = nullptr;
#endif

private:
	void InternalReset();
};

/* UCompositingMediaCaptureOutput
 *****************************************************************************/

class UMediaOutput;
class UMediaCapture;
class UCompositingElementTransform;
class UTextureRenderTarget2D;

UCLASS()
class COMPOSURE_API UCompositingMediaCaptureOutput : public UColorConverterOutputPass
{
	GENERATED_BODY()

public:
	UCompositingMediaCaptureOutput();

	UPROPERTY(EditAnywhere, Category="Compositing Pass", BlueprintReadWrite, meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	TObjectPtr<UMediaOutput> CaptureOutput;

	/** */
	bool IsCapturing() const;

public:
	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

	//~ Begin UCompositingElementOutput interface
	virtual void Reset_Implementation() override;
	virtual void RelayOutput_Implementation(UTexture* FinalResult, UComposurePostProcessingPassProxy* PostProcessProxy) override;
	virtual void OnDisabled_Implementation() override;
	virtual void OnEnabled_Implementation() override;
	//~ End UCompositingElementOutput interface

protected:
	bool StartCapture(UTextureRenderTarget2D* RenderTarget);
	void StopCapture();

private:
	UPROPERTY(Transient, DuplicateTransient, SkipSerialization)
	TObjectPtr<UMediaCapture> ActiveCapture = nullptr;
};

/* URenderTargetCompositingOutput
 *****************************************************************************/

class UTextureRenderTarget2D;

UCLASS()
class COMPOSURE_API URenderTargetCompositingOutput : public UCompositingElementOutput
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite,Category="Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

public:
	//~ UCompositingElementOutput interface
	virtual void RelayOutput_Implementation(UTexture* FinalResult, UComposurePostProcessingPassProxy* PostProcessProxy) override;
};

/* UEXRFileCompositingOutput
 *****************************************************************************/

UENUM()
enum class EExrCompressionOptions : uint8
{
	Compressed   = 0,
	Uncompressed = 1,
};

UCLASS()
class COMPOSURE_API UEXRFileCompositingOutput : public UCompositingElementOutput
{
	GENERATED_BODY()

public:
	UEXRFileCompositingOutput();

	UPROPERTY(EditAnywhere, Category="Compositing Pass", BlueprintReadWrite ,meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	FDirectoryPath OutputDirectiory;

	/**
	 * The format to use for the resulting filename. Extension will be added automatically. Any tokens of the form {token} will be replaced with the corresponding value:
	 * {frame} - The current frame number
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	FString FilenameFormat;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Compositing Pass", meta=(ClampMin=1, UIMin=1, ClampMax=200, UIMax=200, DisplayAfter = "PassName", EditCondition = "bEnabled"))
	FFrameRate OutputFrameRate;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	EExrCompressionOptions Compression = EExrCompressionOptions::Uncompressed;

public:
	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

	//~ Begin UCompositingElementOutput interface
	virtual void Reset_Implementation() override;
	virtual void RelayOutput_Implementation(UTexture* FinalResult, UComposurePostProcessingPassProxy* PostProcessProxy) override;
	virtual void OnDisabled_Implementation() override;
	//~ End UCompositingElementOutput interface

private:
	void InternalReset();

	int32  FrameNumber = 0;
	double SecondsSinceLastCapture = 0.0;
};