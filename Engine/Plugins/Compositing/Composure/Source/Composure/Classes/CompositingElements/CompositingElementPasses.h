// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "CompositingElements/InheritedCompositingTargetPool.h" // for FInheritedTargetPool
#include "Engine/TextureRenderTarget2D.h" // for ETextureRenderTargetFormat
#include "PixelFormat.h" // for EPixelFormat
#include "CompositingElementPasses.generated.h"

enum class ECompPassConstructionType;

/* UCompositingElementPass
 *****************************************************************************/

UCLASS(editinlinenew, Abstract)
class COMPOSURE_API UCompositingElementPass : public UObject
{
	GENERATED_BODY()

protected:
	UPROPERTY(EditAnywhere, BlueprintGetter = IsPassEnabled, BlueprintSetter = SetPassEnabled, Category = "Compositing Pass")
	bool bEnabled = true;
	
public:
	UCompositingElementPass();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Compositing Pass", meta = (EditCondition = "bEnabled"))
	FName PassName;

#if WITH_EDITOR
	ECompPassConstructionType ConstructionMethod;
#endif

public:
	UFUNCTION(BlueprintNativeEvent, Category = "Composure|Compositing Pass", meta=(CallInEditor = "true"))
	void OnFrameBegin(bool bCameraCutThisFrame);

	UFUNCTION(BlueprintNativeEvent, Category = "Composure|Compositing Pass", meta=(CallInEditor = "true"))
	void OnFrameEnd();

	UFUNCTION(BlueprintNativeEvent, Category = "Composure|Compositing Pass", meta=(CallInEditor = "true"))
	void Reset();

	UFUNCTION(BlueprintGetter)
	bool IsPassEnabled() const { return bEnabled; }

	UFUNCTION(BlueprintSetter)
	void SetPassEnabled(bool bSetEnabledTo = true);

public: 
	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

protected:
	void SetRenderTargetPool(const FInheritedTargetPool& TargetPool);

	UFUNCTION(BlueprintNativeEvent, Category = "Composure|Compositing Pass", meta=(CallInEditor = "true"))
	void OnDisabled();
	UFUNCTION(BlueprintNativeEvent, Category = "Composure|Compositing Pass", meta=(CallInEditor = "true"))
	void OnEnabled();

	UFUNCTION(BlueprintCallable, Category = "Composure|Compositing Pass"/*, meta = (BlueprintProtected = "true")*/)
	UTextureRenderTarget2D* RequestRenderTarget(FIntPoint Dimensions, ETextureRenderTargetFormat Format);
	UTextureRenderTarget2D* RequestRenderTarget(FIntPoint Dimensions, EPixelFormat Format);

	UFUNCTION(BlueprintCallable, Category = "Composure|Compositing Pass"/*, meta = (BlueprintProtected = "true")*/)
	UTextureRenderTarget2D* RequestNativelyFormattedTarget(float RenderScale = 1.f);

	UFUNCTION(BlueprintCallable, Category = "Composure|CompositingPass"/*, meta = (BlueprintProtected = "true")*/)
	bool ReleaseRenderTarget(UTextureRenderTarget2D* AssignedTarget);

protected:
	FInheritedTargetPool SharedTargetPool;
};

/* UCompositingElementInput
 *****************************************************************************/

class UTexture;

UCLASS(BlueprintType, Blueprintable, Abstract)
class COMPOSURE_API UCompositingElementInput : public UCompositingElementPass
{
	GENERATED_BODY()

public:
	/** 
	 * Marks this pass for 'internal use only' - prevents it from being exposed to parent elements. 
	 * When set, render target resources used by this element will be reused. For inputs, all 'Intermediate'
	 * passes are available to the first transform pass, and released after that.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (DisplayName = "Intermediate", EditCondition = "bEnabled", DisplayAfter = "EndOfCategory"), AdvancedDisplay)
	bool bIntermediate = true;

	UTexture* GenerateInput(const FInheritedTargetPool& InheritedPool);
protected:
	UFUNCTION(BlueprintNativeEvent, Category="Composure|Compositing Pass", meta = (CallInEditor))
	UTexture* GenerateInput();
};

/* UCompositingElementTransform
 *****************************************************************************/

class UTexture;
class UComposurePostProcessingPassProxy;
class ICompositingTextureLookupTable;
class ACameraActor;

UCLASS(BlueprintType, Blueprintable, Abstract)
class COMPOSURE_API UCompositingElementTransform : public UCompositingElementPass
{
	GENERATED_BODY()

public:
	/** 
	 * Marks this pass for 'internal use only' - prevents it from being exposed to parent elements. 
	 * When set, render target resources used by this element will be reused. For transforms, all 'Intermediate'
	 * passes are available to the next transform pass, and released after that.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (DisplayName = "Intermediate", EditCondition = "bEnabled", DisplayAfter = "EndOfCategory"), AdvancedDisplay)
	bool bIntermediate = true;

	UTexture* ApplyTransform(UTexture* Input, ICompositingTextureLookupTable* PrePassLookupTable, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera, const FInheritedTargetPool& InheritedPool);
protected:
	UFUNCTION(BlueprintNativeEvent, Category="Composure|Compositing Pass", meta = (CallInEditor))
	UTexture* ApplyTransform(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera);

protected:
	UFUNCTION(BlueprintCallable, Category = "Composure|Compositing Pass", meta = (BlueprintProtected = "true"))
	UTexture* FindNamedPrePassResult(FName PassLookupName);

	ICompositingTextureLookupTable* PrePassLookupTable = nullptr;
};

/* UCompositingElementOutput
 *****************************************************************************/

class UTexture;
class UComposurePostProcessingPassProxy;

UCLASS(BlueprintType, Blueprintable, Abstract)
class COMPOSURE_API UCompositingElementOutput : public UCompositingElementPass
{
	GENERATED_BODY()

public:
	void RelayOutput(UTexture* FinalResult, UComposurePostProcessingPassProxy* PostProcessProxy, const FInheritedTargetPool& InheritedPool);
protected:
	UFUNCTION(BlueprintNativeEvent, Category="Composure|Compositing Pass", meta = (CallInEditor))
	void RelayOutput(UTexture* FinalResult, UComposurePostProcessingPassProxy* PostProcessProxy);
};
