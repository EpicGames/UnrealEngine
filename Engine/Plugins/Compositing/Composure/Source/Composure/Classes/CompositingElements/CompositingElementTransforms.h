// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompositingElements/CompositingElementPasses.h"
#include "CompositingElements/CompositingMaterialPass.h"
#include "Engine/Scene.h" // for FColorGradingSettings, FFilmStockSettings
#include "OpenColorIOColorSpace.h"

#include "CompositingElementTransforms.generated.h"

/* UCompositingPostProcessPass
 *****************************************************************************/

class FCompositingTargetSwapChain;
class UComposurePostProcessPassPolicy;

UCLASS(BlueprintType, Blueprintable)
class COMPOSURE_API UCompositingPostProcessPass : public UCompositingElementTransform
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	float RenderScale = 1.f;

	UPROPERTY(EditAnywhere, Instanced, BlueprintReadOnly, Category = "Compositing Pass", meta = (DisplayAfter = "RenderScale"))
	TArray<TObjectPtr<UComposurePostProcessPassPolicy>> PostProcessPasses;

public:
	//~ Begin UCompositingElementTransform interface
	virtual UTexture* ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera) override;

protected:
	void RenderPostPassesToSwapChain(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, FCompositingTargetSwapChain& TargetSwapChain);
};

/* UCompositingElementMaterialPass
 *****************************************************************************/

UCLASS(BlueprintType, Blueprintable, editinlinenew)
class COMPOSURE_API UCompositingElementMaterialPass : public UCompositingPostProcessPass
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Compositing Pass", meta = (ShowOnlyInnerProperties, DisplayAfter = "PassName", EditCondition = "bEnabled"))
	FCompositingMaterial Material;

public:
	//~ Begin UCompositingElementTransform interface
	virtual UTexture* ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera) override;
	//~ End UCompositingElementTransform interface

	/**
	 * Set the material interface used by current material pass. 
	 * @param NewMaterial            The new material interface users want to set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Compositing Pass")
	void SetMaterialInterface(UMaterialInterface* NewMaterial);

	/**
	 * Set the parameter mappings between texture parameters and composure layers. Users can not create new entries into the map as the keys are read only.
	 * Invalid Texture parameter names will result in a failed setting operation. 
	 * @param TextureParamName       The name of the texture parameter inside the material interface. Used as key.
	 * @param ComposureLayerName     The name of the composure layer the texture parameter is mapped to. Used as value.
	 * @return bool                  True if set operation is successful. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Compositing Pass")
	bool SetParameterMapping(FName TextureParamName, FName ComposureLayerName);

protected:
	UFUNCTION(BlueprintImplementableEvent)
	void ApplyMaterialParams(UMaterialInstanceDynamic* MID);
};

/* UCompositingTonemapPass
 *****************************************************************************/

class UComposureTonemapperPassPolicy;

UCLASS(BlueprintType, Blueprintable)
class COMPOSURE_API UCompositingTonemapPass : public UCompositingElementTransform
{
	GENERATED_BODY()

public:
	/** Color grading settings. */
	UPROPERTY(Interp, Category = "Compositing Pass",meta = (ShowOnlyInnerProperties, DisplayAfter = "PassName", EditCondition = "bEnabled"))
	FColorGradingSettings ColorGradingSettings;
	
	/** Film stock settings. */
	UPROPERTY(Interp, Category = "Compositing Pass",meta = (ShowOnlyInnerProperties, DisplayAfter = "PassName", EditCondition = "bEnabled"))
	FFilmStockSettings FilmStockSettings;

	/** in percent, Scene chromatic aberration / color fringe (camera imperfection) to simulate an artifact that happens in real-world lens, mostly visible in the image corners. */
	UPROPERTY(Interp, Category = "Compositing Pass", meta = (UIMin = "0.0", UIMax = "5.0", DisplayAfter = "PassName", EditCondition = "bEnabled"))
	float ChromaticAberration = 0.0f;

public:
	//~ Begin UCompositingElementTransform interface
	virtual UTexture* ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera) override;

private:
	UPROPERTY(Transient, DuplicateTransient, SkipSerialization)
	TObjectPtr<UComposureTonemapperPassPolicy> TonemapPolicy;
};

/* UMultiPassChromaKeyer
 *****************************************************************************/

class UMediaBundle;
class UMaterialInstanceDynamic;
class UMaterialInterface;

UCLASS(BlueprintType, Blueprintable)
class COMPOSURE_API UMultiPassChromaKeyer : public UCompositingElementTransform
{
	GENERATED_BODY()

public:
	UMultiPassChromaKeyer();

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	TArray<FLinearColor> KeyColors;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (ShowOnlyInnerProperties, DisplayAfter = "PassName", EditCondition = "bEnabled"))
	FCompositingMaterial KeyerMaterial;

public:
	//~ Begin UCompositingElementTransform interface
	virtual UTexture* ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera) override;

private:
	UPROPERTY(Transient)
	TObjectPtr<UTexture> DefaultWhiteTexture = nullptr;
};


/* UMultiPassDespill
 *****************************************************************************/

class UMediaBundle;
class UMaterialInstanceDynamic;
class UMaterialInterface;

UCLASS(BlueprintType, Blueprintable)
class COMPOSURE_API UMultiPassDespill : public UCompositingElementTransform
{
	GENERATED_BODY()

public:
	UMultiPassDespill();

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	TArray<FLinearColor> KeyColors;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Compositing Pass", meta = (ShowOnlyInnerProperties, DisplayAfter = "PassName", EditCondition = "bEnabled"))
	FCompositingMaterial KeyerMaterial;

public:
	//~ Begin UCompositingElementTransform interface
	virtual UTexture* ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera) override;

private:
	UPROPERTY(Transient)
	TObjectPtr<UTexture> DefaultWhiteTexture = nullptr;
};

/* UAlphaTransformPass
 *****************************************************************************/

class UMaterialInstanceDynamic;

UCLASS(noteditinlinenew, hidedropdown)
class COMPOSURE_API UAlphaTransformPass : public UCompositingElementTransform
{
	GENERATED_BODY()

public:
	UAlphaTransformPass();

	UPROPERTY(EditAnywhere,Category = "Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	float AlphaScale = 1.f;

public:
	//~ Begin UCompositingElementTransform interface
	virtual UTexture* ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera) override;
	//~ End UCompositingElementTransform interface

private:
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> DefaultMaterial;
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> AlphaTransformMID;
};


/* UCompositingOpenColorIOPass
*****************************************************************************/

UCLASS(BlueprintType, Blueprintable, meta = (DisplayName = "OpenColorIO Pass"))
class COMPOSURE_API UCompositingOpenColorIOPass : public UCompositingElementTransform
{
	GENERATED_BODY()

public:
	/** Color grading settings. */
	UPROPERTY(Interp, Category = "OpenColorIO Settings",meta = (ShowOnlyInnerProperties, DisplayAfter = "PassName", EditCondition = "bEnabled"))
	FOpenColorIOColorConversionSettings ColorConversionSettings;

public:
	//~ Begin UCompositingElementTransform interface
	virtual UTexture* ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera) override;
};