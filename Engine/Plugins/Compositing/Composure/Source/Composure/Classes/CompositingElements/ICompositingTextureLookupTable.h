// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "ICompositingTextureLookupTable.generated.h"

class UTexture;

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UCompositingTextureLookupTable : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class COMPOSURE_API ICompositingTextureLookupTable
{
	GENERATED_IINTERFACE_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Composure")
	virtual bool FindNamedPassResult(FName LookupName, UTexture*& OutTexture) const = 0;
};
