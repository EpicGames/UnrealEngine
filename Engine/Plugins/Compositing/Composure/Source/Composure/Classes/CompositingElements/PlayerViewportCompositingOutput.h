// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Camera/CameraModifier.h"
#include "Engine/BlendableInterface.h"
#include "CompositingElements/CompositingElementOutputs.h" // for UColorConverterOutputPass
#include "PlayerViewportCompositingOutput.generated.h"

class APlayerController;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UPlayerCompOutputCameraModifier;

/* UPlayerViewportCompositingOutput
 *****************************************************************************/

UCLASS()
class COMPOSURE_API UPlayerViewportCompositingOutput : public UColorConverterOutputPass, public IBlendableInterface
{
	GENERATED_BODY()

public:
	 UPlayerViewportCompositingOutput();
	~UPlayerViewportCompositingOutput();

	UPROPERTY(EditAnywhere, Category="Compositing Pass", meta = (DisplayAfter = "PassName", EditCondition = "bEnabled"))
	int32 PlayerIndex = 0;

	/** Enable or disable the tone curve (and expand gamut) when the tonemap pass settings are passed down to the main viewport post-processing. */
	UPROPERTY(EditAnywhere, Category = "Compositing Pass", meta = (DisplayAfter = "ColorConversion", EditCondition = "bEnabled"))
	bool ApplyToneCurve = false;

public:
	//~ Begin UCompositingElementOutput interface
	virtual void OnFrameBegin_Implementation(bool bCameraCutThisFrame) override;
	virtual void RelayOutput_Implementation(UTexture* RenderResult, UComposurePostProcessingPassProxy* PostProcessProxy) override;
	virtual void Reset_Implementation() override;
	//~ End UCompositingElementOutput interface

	//~ Begin IBlendableInterface interface
	virtual void OverrideBlendableSettings(FSceneView& View, float Weight) const override;
	//~ End IBlendableInterface interface


#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
protected:
	bool OverridePlayerCamera(int32 PlayerIndex);
	void ClearViewportOverride();
	UMaterialInstanceDynamic* GetBlendableMID();

	friend class UPlayerCompOutputCameraModifier;
	bool UseBuiltInColorConversion() const;

private:
	int32 ActiveOverrideIndex = INDEX_NONE;

	UPROPERTY(Transient, DuplicateTransient, SkipSerialization)
	TObjectPtr<UPlayerCompOutputCameraModifier> ActiveCamModifier;

	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> TonemapperBaseMat;
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> PreTonemapBaseMat;

	UPROPERTY(Transient, DuplicateTransient, SkipSerialization)
	TObjectPtr<UMaterialInstanceDynamic> ViewportOverrideMID;

	TWeakObjectPtr<APlayerController> TargetedPlayerController;
};

/* UPlayerCompOutputCameraModifier
 *****************************************************************************/

UCLASS(NotBlueprintType)
class UPlayerCompOutputCameraModifier : public UCameraModifier
{
	GENERATED_BODY()

public:
	void SetOwner(UPlayerViewportCompositingOutput* Owner);

public:
	//~ UCameraModifier interface
	virtual bool ModifyCamera(float DeltaTime, FMinimalViewInfo& InOutPOV) override;	

private:
	UPROPERTY(Transient)
	TObjectPtr<UPlayerViewportCompositingOutput> Owner;
};

