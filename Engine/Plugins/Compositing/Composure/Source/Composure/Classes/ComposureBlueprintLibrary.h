// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "ComposurePostMoves.h"
#include "ComposureUVMap.h"
#include "CompositingElement.h"

#include "ComposureBlueprintLibrary.generated.h"

class UCameraComponent;
class USceneCaptureComponent2D;

UCLASS(meta=(ScriptName="ComposureLibrary"))
class COMPOSURE_API UComposureBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** Creates a Player Compositing Target which you can modify during gameplay. */
	UFUNCTION(BlueprintCallable, Category = "Composure", meta = (WorldContext = "WorldContextObject"))
	static class UComposurePlayerCompositingTarget* CreatePlayerCompositingTarget(UObject* WorldContextObject);

	/** 
	 * Returns a non-centered projection matrix.
	 * @param HorizontalFOVAngle The desired horizontal FOV in degrees.
	 * @param AspectRatio The desired aspect ratio.
	 */
	UFUNCTION(BlueprintPure, Category = "Composure")
	static void GetProjectionMatrixFromPostMoveSettings(
		const FComposurePostMoveSettings& PostMoveSettings, float HorizontalFOVAngle, float AspectRatio, FMatrix& ProjectionMatrix);
		
	/**
	 * Returns UV transformation matrix and its inversed to crop.
	 * @param AspectRatio The desired aspect ratio.
	 */
	UFUNCTION(BlueprintPure, Category = "Composure")
	static void GetCroppingUVTransformationMatrixFromPostMoveSettings(
		const FComposurePostMoveSettings& PostMoveSettings, float AspectRatio,
		FMatrix& CropingUVTransformationMatrix, FMatrix& UncropingUVTransformationMatrix);

	/** Sets parameters of a material that uses Composure's MF_UVMap_SampleLocation material function. */
	UFUNCTION(BlueprintCallable, Category = "Composure")
	static void SetUVMapSettingsToMaterialParameters(
		const FComposureUVMapSettings& UVMapSettings, class UMaterialInstanceDynamic* Material)
	{
		UVMapSettings.SetMaterialParameters(Material);
	}
	
	/**
	 * Converts displacement encoding parameters to decoding parameters.
	 * Can also be used to convert displacement decoding parameters to encoding parameters.
	 */
	UFUNCTION(BlueprintPure, Category = "Composure")
	static void InvertUVDisplacementMapEncodingParameters(
		const FVector2D& In, FVector2D& Out)
	{
		Out = FComposureUVMapSettings::InvertEncodingParameters(In);
	}

	/** Returns the red and green channel factors from percentage of chromatic aberration. */
	UFUNCTION(BlueprintPure, Category = "Composure")
	static void GetRedGreenUVFactorsFromChromaticAberration(float ChromaticAberrationAmount, FVector2D& RedGreenUVFactors);

	/** Returns display gamma of a given player camera manager, or 0 if no scene viewport attached. */
	UFUNCTION(BlueprintPure, Category = "Composure")
	static void GetPlayerDisplayGamma(const APlayerCameraManager* PlayerCameraManager, float& DisplayGamma);

	UFUNCTION(BlueprintCallable, Category = "Composure")
	static void CopyCameraSettingsToSceneCapture(UCameraComponent* SrcCamera, USceneCaptureComponent2D* DstCaptureComponent, float OriginalFocalLength, float OverscanFactor = 1.0f);

	/**
	 * Create a new Composure in the level without any parenting relationship.
	 * @param ElementName              The name for the newly created composure element
	 * @param ClassType                The type for the new composure element
	 * @param LevelContext             The level context of current level. Default value is nullptr.
	 * @return CompositingElement      The created composure element.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Manager", meta = (DeterminesOutputType = "ClassType"))
	static ACompositingElement* CreateComposureElement(const FName ElementName,UPARAM(meta=(AllowAbstract="false")) TSubclassOf<ACompositingElement> ClassType, AActor* LevelContext = nullptr);

	/**
	 * Get a specific composure element 
	 * @param ElementName              The name of the composure element that we want to get.
	 * @return CompositingElement      The composure element found. It can be nullptr if there is no composure element matches the input name.
	 */
	UFUNCTION(BlueprintPure, Category = "Composure|Manager")
	static ACompositingElement* GetComposureElement(const FName ElementName);

	/**
	 * Delete a specific composure element without evoking prompt window. Will delete all of its children as well.
	 * @param ElementToDelete           The name of the composure element that we want to delete.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Manager")
	static void DeleteComposureElementAndChildren(const FName ElementToDelete);

	/**
	 * Rename a specific composure element
	 * @param OriginalElementName       The name of the composure element that we want to rename.
	 * @param NewElementName            The new name for the composure element.
	 * @return bool                     Whether the renaming operation is successful or not.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Manager")
	static bool RenameComposureElement(const FName OriginalElementName, const FName NewElementName);

	/**
	 * Attach one composure element as the child to another composure element in the scene.
	 * @param ParentName                The name of the parent composure element.
	 * @param ChildName                 The name of the child composure element.
	 * @return bool                     Whether the attaching process is successful or not.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Manager")
	static bool AttachComposureElement(const FName ParentName, const FName ChildName);

	/**
	 * Determines if the specified element is being rendered by the hidden compositing viewport.
	 * @param  CompElement	The element actor you're querying for
	 * @return True if the game-thread is in the middle of queuing the specified element.
	 */
	UFUNCTION(BlueprintPure, Category = "Composure|Manager")
	static bool IsComposureElementDrawing(ACompositingElement* CompElement);

	/**
	 * Request redrawing the compositing editor viewport  if it is valid.
	 * If it is invalid, this function will create a new view port client. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Manager")
	static void RequestRedrawComposureViewport();

	/**
	 * Re-queries the scene for element actors and rebuilds the authoritative list used by the editor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Composure|Manager")
	static void RefreshComposureElementList();
};
