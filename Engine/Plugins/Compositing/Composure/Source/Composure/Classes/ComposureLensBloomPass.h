// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/Scene.h"
#include "ComposurePostProcessPass.h"
#include "ComposurePostProcessingPassProxy.h" // for UComposurePostProcessPassPolicy
#include "ComposureLensBloomPass.generated.h"


/**
 * Bloom only pass implemented on top of the in-engine bloom.
 */
UCLASS(hidecategories = (Collision, Object, Physics, SceneComponent, Transform), ClassGroup = "Composure", editinlinenew, meta = (BlueprintSpawnableComponent))
class COMPOSURE_API UComposureLensBloomPass : public UComposurePostProcessPass
{
	GENERATED_UCLASS_BODY()

public:

	/** Bloom settings. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens Bloom Settings")
	FLensBloomSettings Settings;
	

	/** Sets a custom tonemapper replacing material instance. */
	UFUNCTION(BlueprintCallable, Category = "Lens Bloom Settings")
	void SetTonemapperReplacingMaterial(UMaterialInstanceDynamic* Material);

	
	/** 
	 * Blurs the input into the output.
	 */
	UFUNCTION(BlueprintCallable, Category = "Outputs")
	void BloomToRenderTarget();

private:
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> TonemapperReplacingMID;
};

/**
 * Bloom only rules used for configuring how UComposurePostProcessingPassProxy executes
 */
UCLASS(BlueprintType, Blueprintable, editinlinenew, meta=(DisplayName="Lens Bloom Pass"))
class COMPOSURE_API UComposureLensBloomPassPolicy : public UComposurePostProcessPassPolicy
{
	GENERATED_BODY()
	UComposureLensBloomPassPolicy();

public:
	/** Bloom settings. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens Bloom Settings")
	FLensBloomSettings Settings;

	UPROPERTY(EditAnywhere, Category = "Tonemapper Settings", AdvancedDisplay)
	TObjectPtr<UMaterialInterface> ReplacementMaterial;

	UPROPERTY(EditAnywhere, Category = "Tonemapper Settings", AdvancedDisplay)
	FName BloomIntensityParamName;

public:
	//~ UComposurePostProcessPassPolicy interface
	void SetupPostProcess_Implementation(USceneCaptureComponent2D* SceneCapture, UMaterialInterface*& OutTonemapperOverride) override;

private:
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> TonemapperReplacmentMID;
};
