// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "ComposurePipelineBaseActor.generated.h"

class FComposureViewExtension;

/**
 * Actor designed to implement compositing pipeline in a blueprint.
 */
UCLASS(BlueprintType, Blueprintable, config=Engine, meta=(ShortTooltip="Actor designed to implement compositing pipeline in a blueprint."))
class COMPOSURE_API AComposurePipelineBaseActor
	: public AActor
{
	GENERATED_UCLASS_BODY()
public:
	/** 
	 * When set, we'll call EnqueueRendering() each frame automatically. If left 
	 * off, it is up to the user to manually call their composure rendering. 
	 * Toggle this on/off at runtime to enable/disable this pipeline.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetAutoRun, Category="Composure|Ticking", meta = (DisplayAfter = "bEnableChildElementsAndSelf"))
	bool bAutoRun;

#if WITH_EDITORONLY_DATA
	/** With bAutoRun, this will run EnqueueRendering() in editor - enqueuing render calls along with Editor scene rendering. */
	UPROPERTY(EditAnywhere, Category="Composure|Ticking", meta = (EditCondition = "bAutoRun", DisplayAfter = "bAutoRun"))
	bool bRunInEditor;
#endif 

	UFUNCTION(BlueprintSetter)
	virtual void SetAutoRun(bool bNewAutoRunVal) { bAutoRun = bNewAutoRunVal; }

	UFUNCTION(BlueprintGetter)
	bool AreChildrenAndSelfAutoRun() const { return bAutoRunChildElementsAndSelf; }

public:	
	/** 
	 *
	 */
	UFUNCTION(BlueprintNativeEvent, Category="Composure", meta = (CallInEditor = "true"))
	bool IsActivelyRunning() const;
#if WITH_EDITOR
	bool IsAutoRunSuspended() const;
#endif

	/** 
	 * Entry point for a composure Blueprint to do its render enqueuing from.
	 */
	UFUNCTION(BlueprintNativeEvent, Category="Composure", meta = (CallInEditor = "true"))
	void EnqueueRendering(bool bCameraCutThisFrame);

	virtual int32 GetRenderPriority() const { return 0; }

	UFUNCTION(BlueprintSetter, Category = "Composure|Element")
	virtual void SetAutoRunChildrenAndSelf(bool bAutoRunChildAndSelf) {}

public:
#if WITH_EDITOR
	//~ AActor interface
	virtual void RerunConstructionScripts() override;
#endif

protected:
	/**
	 * When set to false, all composure elements including itself's rendering will not automatically be called in the pipeline.
	 * When set to true, all of its children and its self's rendering will be called every frame.
	 */
	UPROPERTY(EditAnywhere, BlueprintGetter = AreChildrenAndSelfAutoRun, BlueprintSetter = SetAutoRunChildrenAndSelf,Category = "Composure|Ticking", meta = (DisplayName = "Auto run children and self"))
	bool bAutoRunChildElementsAndSelf = true;
	
private: 
	TSharedPtr<FComposureViewExtension, ESPMode::ThreadSafe> ViewExtension;
};
