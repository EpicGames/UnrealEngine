// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ShowFlags.h"
#include "Components/ActorComponent.h"
#include "UObject/Interface.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "ComposurePlayerCompositingInterface.h"
#include "Framework/Commands/UICommandList.h"
#include "EditorSupport/CompEditorImagePreviewInterface.h"
#include "ComposurePlayerCompositingTarget.generated.h"

class APlayerCameraManager;
class UTextureRenderTarget2D;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class UComposurePlayerCompositingCameraModifier;

#if WITH_EDITOR
class SWidget;
class SImage;
struct FSlateBrush;
struct FMinimalViewInfo;
#endif

/**
 * Object to bind to a APlayerCameraManager with a UTextureRenderTarget2D to be used as a player's render target.
 * @TODO-BADGER: deprecate this (UComposurePlayerCompositingTarget) once we're comfortable using the new UComposureCompositingTargetComponent in its place
 */
UCLASS(BlueprintType)
class COMPOSURE_API UComposurePlayerCompositingTarget : public UObject, public IComposurePlayerCompositingInterface
{
	GENERATED_UCLASS_BODY()

	~UComposurePlayerCompositingTarget();

public:

	// Current player camera manager the target is bind on.
	UFUNCTION(BlueprintCallable, Category = "Player Compositing target")
	APlayerCameraManager* GetPlayerCameraManager() const
	{
		return PlayerCameraManager;
	}

	// Set player camera manager to bind the render target to.
	UFUNCTION(BlueprintCallable, Category = "Player Compositing target")
	APlayerCameraManager* SetPlayerCameraManager(APlayerCameraManager* PlayerCameraManager);

	// Set the render target of the player.
	UFUNCTION(BlueprintCallable, Category = "Player Compositing target")
	void SetRenderTarget(UTextureRenderTarget2D* RenderTarget);


	// Begins UObject
	virtual void FinishDestroy() override;
	// Ends UObject


private:
	// Current player camera manager the target is bind on.
	UPROPERTY(Transient)
	TObjectPtr<APlayerCameraManager> PlayerCameraManager;

	// Underlying player camera modifier
	UPROPERTY(Transient)
	TObjectPtr<class UComposurePlayerCompositingCameraModifier> PlayerCameraModifier;
	
	// Post process material that replaces the tonemapper to dump the player's render target.
	UPROPERTY(Transient)
	TObjectPtr<class UMaterialInstanceDynamic> ReplaceTonemapperMID;

	// Backup of the engine showflags to restore when unbinding the compositing target from the camera manager.
	FEngineShowFlags EngineShowFlagsBackup;


	// Entries called by PlayerCameraModifier.
	virtual void OverrideBlendableSettings(class FSceneView& View, float Weight) const override;


	friend class UComposurePlayerCompositingCameraModifier;
};

/* UComposureCompositingTargetComponent
 *****************************************************************************/

class SCompElementPreviewPane;

/**
 * Component intended to replace UComposurePlayerCompositingTarget - a object to bind to a APlayerCameraManager 
 * with a UTextureRenderTarget2D to be used as a player's render target.
 * Made into a component so we can hook into preview rendering in editor.
 */
UCLASS(ClassGroup = Rendering, Blueprintable, BlueprintType, meta = (BlueprintSpawnableComponent))
class COMPOSURE_API UComposureCompositingTargetComponent : public UActorComponent, public ICompEditorImagePreviewInterface
{
	GENERATED_UCLASS_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Composure|CompositingTarget")
	void SetDisplayTexture(UTexture* DisplayTexture);

	UFUNCTION(BlueprintCallable, Category = "Composure|CompositingTarget")
	UTexture* GetDisplayTexture() const { return DisplayTexture; }

#if WITH_EDITOR
	bool IsPreviewing() const;
	void SetUseImplicitGammaForPreview(const bool bInUseImplicitGammaOnPreview) { bUseImplicitGammaOnPreview = bInUseImplicitGammaOnPreview; }
#endif 

public:
	//~ Begin UActorComponent interface
#if WITH_EDITOR
	virtual bool GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut) override;
	virtual TSharedPtr<SWidget> GetCustomEditorPreviewWidget() override;
#endif 
	//~ End UActorComponent interface

	//~ ICompEditorImagePreviewInterface interface
#if WITH_EDITOR
	virtual void OnBeginPreview() override;
	virtual UTexture* GetEditorPreviewImage() override;
	virtual void OnEndPreview() override;
	virtual bool UseImplicitGammaForPreview() const override { return bUseImplicitGammaOnPreview; }
#endif
	//~ End ICompEditorImagePreviewInterface interface

private:
	UPROPERTY(Transient)
	TObjectPtr<UTexture> DisplayTexture;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<UTexture> CompilerErrImage;

	int32 PreviewCount = 0;
	bool bUseImplicitGammaOnPreview = true;
	bool bHasCompilerError = false;
#endif
};
