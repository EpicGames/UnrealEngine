// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Components/SceneComponent.h"
#include "ComposurePostProcessPass.generated.h"


class UMaterialInterface;
class UMaterialInstanceDynamic;
class UTextureRenderTarget2D;
class USceneCaptureComponent2D;

class UComposurePostProcessBlendable;


/**
 * In engine post process based pass.
 */
UCLASS()
class COMPOSURE_API UComposurePostProcessPass
	: public USceneComponent
{
	GENERATED_UCLASS_BODY()

public:

	/** 
	 * Sets a custom setup post process material. The material location must be set at BeforeTranslucency.
	 */
	UFUNCTION(BlueprintCallable, Category = "Inputs")
	void SetSetupMaterial(UMaterialInterface* Material);
	
	/** Gets current setup material. */
	UFUNCTION(BlueprintPure, Category = "Inputs")
	UMaterialInterface* GetSetupMaterial() const
	{
		return SetupMaterial;
	}

	/** 
	 * Gets current output render target.
	 */
	UFUNCTION(BlueprintCallable, Category = "Outputs")
	UTextureRenderTarget2D* GetOutputRenderTarget() const;
	
	/** 
	 * Sets current output render target.
	 */
	UFUNCTION(BlueprintCallable, Category = "Outputs")
	void SetOutputRenderTarget(UTextureRenderTarget2D* RenderTarget);
	

public:
	//~ UActorComponent interface
	virtual void Activate(bool bReset = false) override;
	virtual void Deactivate() override;
	virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;


protected:
	// Underlying scene capture.
	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<USceneCaptureComponent2D> SceneCapture;

	// Blendable interface to intercept the OverrideBlendableSettings.
	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<UComposurePostProcessBlendable> BlendableInterface;

	// Setup post process material.
	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<UMaterialInterface> SetupMaterial;

	// Internal material that replace the tonemapper to output linear color space.
	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<UMaterialInterface> TonemapperReplacement;

	// Called by UComposurePostProcessBlendable::OverrideBlendableSettings.
	void OverrideBlendableSettings(class FSceneView& View, float Weight) const;


	friend class UComposurePostProcessBlendable;

};
