// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ComposurePostProcessPass.h"
#include "ComposurePostProcessingPassProxy.generated.h"

class UMaterialInstanceDynamic;
class UTexture;

/**
 * Abstract base class for setting up post passes. Used in conjuntion with UComposurePostProcessingPassProxy.
 */
UCLASS(Blueprintable, BlueprintType, Abstract)
class COMPOSURE_API UComposurePostProcessPassPolicy : public UObject
{
	GENERATED_BODY()

public:
	UComposurePostProcessPassPolicy() {};

	/** */
	UFUNCTION(BlueprintNativeEvent, Category = "Composure")
	void SetupPostProcess(USceneCaptureComponent2D* SceneCapture, UMaterialInterface*& TonemapperOverride);
};

/**
 * Generic component class which takes a UComposurePostProcessPassPolicy and 
 * executes it, enqueuing a post-process pass for the render frame.
 */
UCLASS(hidecategories = (Collision, Object, Physics, SceneComponent, Transform), ClassGroup = "Composure", editinlinenew, meta = (BlueprintSpawnableComponent))
class COMPOSURE_API UComposurePostProcessingPassProxy : public UComposurePostProcessPass
{
	GENERATED_UCLASS_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Composure")
	void Execute(UTexture* PrePassInput, UComposurePostProcessPassPolicy* PostProcessPass);

	//~ Begin UObject interface
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	//~ End UObject interface

private:
	UPROPERTY(Transient, DuplicateTransient, SkipSerialization)
	TObjectPtr<UMaterialInstanceDynamic> SetupMID;
};
