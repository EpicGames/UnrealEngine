// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/Scene.h"
#include "ComposurePostProcessPass.h"
#include "ComposurePostProcessingPassProxy.h" // for UComposurePostProcessPassPolicy
#include "ComposureTonemapperPass.generated.h"

struct FPostProcessSettings;

class FComposureTonemapperUtils
{
public:
	static void ApplyTonemapperSettings(const FColorGradingSettings& ColorGradingSettings, const FFilmStockSettings& FilmStockSettings, const float ChromaticAberration, FPostProcessSettings& OutSettings);
};

/**
 * Tonemapper only pass implemented on top of the in-engine tonemapper.
 */
UCLASS(hidecategories = (Collision, Object, Physics, SceneComponent, Transform), ClassGroup = "Composure", editinlinenew, meta = (BlueprintSpawnableComponent))
class COMPOSURE_API UComposureTonemapperPass : public UComposurePostProcessPass
{
	GENERATED_UCLASS_BODY()

public:

	/** Color grading settings. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Settings")
	FColorGradingSettings ColorGradingSettings;
	
	/** Film stock settings. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Settings")
	FFilmStockSettings FilmStockSettings;

	/** in percent, Scene chromatic aberration / color fringe (camera imperfection) to simulate an artifact that happens in real-world lens, mostly visible in the image corners. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens", meta = (UIMin = "0.0", UIMax = "5.0"))
	float ChromaticAberration;

	
	/** 
	 * Tone map the input into the output.
	 */
	UFUNCTION(BlueprintCallable, Category = "Outputs")
	void TonemapToRenderTarget();
};

/**
 * Tonemapper only rules used for configuring how UComposurePostProcessingPassProxy executes
 */
UCLASS(BlueprintType, Blueprintable, editinlinenew, meta=(DisplayName="Tonemapper Pass"))
class COMPOSURE_API UComposureTonemapperPassPolicy : public UComposurePostProcessPassPolicy
{
	GENERATED_BODY()
	UComposureTonemapperPassPolicy() {}

public:
	/** Color grading settings. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Tonemapper Settings")
	FColorGradingSettings ColorGradingSettings;
	
	/** Film stock settings. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Tonemapper Settings")
	FFilmStockSettings FilmStockSettings;

	/** in percent, Scene chromatic aberration / color fringe (camera imperfection) to simulate an artifact that happens in real-world lens, mostly visible in the image corners. */
	UPROPERTY(Interp, BlueprintReadWrite, Category = "Lens Settings", meta = (UIMin = "0.0", UIMax = "5.0"))
	float ChromaticAberration = 0.0f;

public:
	//~ UComposurePostProcessPassPolicy interface
	void SetupPostProcess_Implementation(USceneCaptureComponent2D* SceneCapture, UMaterialInterface*& OutTonemapperOverride) override;
};

