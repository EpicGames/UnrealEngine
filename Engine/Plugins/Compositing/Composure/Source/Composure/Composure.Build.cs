// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class Composure : ModuleRules
	{
		public Composure(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Engine",
				}
				);
            
            PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"CameraCalibrationCore",
					"CinematicCamera",
					"Core",
                    "CoreUObject",
                    "Engine",
					"LensComponent",
					"MediaIOCore",
					"MovieScene",
					"MovieSceneTracks",
					"OpenColorIO",
					"TimeManagement",
                }
				);

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					// Removed dependency, until the MediaFrameworkUtilities plugin is available for all platforms
                    //"MediaFrameworkUtilities",

					"ImageWriteQueue",
					"MediaAssets",
					"MovieSceneCapture",
					"RHI",
				}
				);

            if (Target.bBuildEditor == true)
            {
                PrivateDependencyModuleNames.AddRange(
					new string[]
                    {
						"ActorLayerUtilities",
						"EditorFramework",
						"Slate",
						"SlateCore",
						"UnrealEd",
					}
					);
            }

			// TODO: Use proper module reference (currently cannot reference editor module for non-editor target)
			PrivateIncludePaths.Add(Path.Combine(GetModuleDirectory("ComposureLayersEditor"), "Public")); // ICompElementManager.h & CompElementEditorModule.h
		}
	}
}
