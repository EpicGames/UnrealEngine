// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingCaptureBase.h"

#include "Camera/CameraActor.h"
#include "CineCameraComponent.h"
#include "Components/SceneCaptureComponent2D.h"
#include "LensComponent.h"

DEFINE_LOG_CATEGORY_STATIC(LogCompositingCaptureBase, Log, All);

ACompositingCaptureBase::ACompositingCaptureBase()
{
	// Create the SceneCapture component and assign its parent to be the RootComponent (the ComposurePostProcessingPassProxy)
	SceneCaptureComponent2D = CreateDefaultSubobject<USceneCaptureComponent2D>(TEXT("SceneCaptureComponent"));
	SceneCaptureComponent2D->SetupAttachment(RootComponent);

	// The SceneCaptureComponent2D default constructor disables TAA, but CG Compositing Elements enable it by default
	SceneCaptureComponent2D->ShowFlags.TemporalAA = true;
}

void ACompositingCaptureBase::PostInitProperties()
{
	Super::PostInitProperties();
	EObjectFlags ExcludeFlags = RF_ArchetypeObject | RF_ClassDefaultObject | RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects | RF_WasLoaded;
	if (!HasAnyFlags(ExcludeFlags))
	{
		ACameraActor* TargetCamera = FindTargetCamera();

		if (TargetCamera)
		{
			ULensComponent* CurrentLensComponent = Cast<ULensComponent>(LensComponentPicker.GetComponent(TargetCamera));
			if (!CurrentLensComponent)
			{
				TInlineComponentArray<ULensComponent*> LensComponents;
				TargetCamera->GetComponents(LensComponents);

				if (LensComponents.Num() > 0)
				{
					SetLens(LensComponents[0]);
				}
			}
		}
	}
}

void ACompositingCaptureBase::UpdateDistortion()
{
	// Get the TargetCameraActor associated with this CG Layer
	ACameraActor* TargetCamera = FindTargetCamera();
	if (TargetCamera == nullptr)
	{
		return;
	}

 	if (UCineCameraComponent* const CineCameraComponent = Cast<UCineCameraComponent>(TargetCamera->GetCameraComponent()))
	{
		// Query the camera calibration subsystem for a handler associated with the TargetCamera and matching the user selection
		ULensDistortionModelHandlerBase* LensDistortionHandler = nullptr;

		ULensComponent* LensComponent = Cast<ULensComponent>(LensComponentPicker.GetComponent(TargetCamera));
		if (LensComponent && LensComponent->GetOwner() == TargetCamera)
		{
			OriginalFocalLength = LensComponent->GetOriginalFocalLength();
			LensDistortionHandler = LensComponent->GetLensDistortionHandler();
		}
		else
		{
			OriginalFocalLength = CineCameraComponent->CurrentFocalLength;
		}

		if (LensDistortionHandler)
		{
			// Get the current distortion MID from the lens distortion handler, and if it has changed, remove the old MID from the scene capture's post process materials
			UMaterialInstanceDynamic* NewDistortionMID = LensDistortionHandler->GetDistortionMID();
			if (LastDistortionMID != NewDistortionMID)
			{
				if (SceneCaptureComponent2D)
				{
					SceneCaptureComponent2D->RemoveBlendable(LastDistortionMID);
				}
			}

			// Cache the latest distortion MID
			LastDistortionMID = NewDistortionMID;

			// If distortion should be applied, add/update the distortion MID to the scene capture's post process materials. Otherwise, remove it.
			if (SceneCaptureComponent2D)
			{
				if (bApplyDistortion && LensComponent->WasDistortionEvaluated())
				{
					OverscanFactor = LensDistortionHandler->GetOverscanFactor();
					SceneCaptureComponent2D->AddOrUpdateBlendable(NewDistortionMID);
				}
				else
				{
					OverscanFactor = 1.0f;
					SceneCaptureComponent2D->RemoveBlendable(NewDistortionMID);
				}
			}
		}
		else
		{
			OverscanFactor = 1.0f;

			if (SceneCaptureComponent2D)
			{
				SceneCaptureComponent2D->RemoveBlendable(LastDistortionMID);
			}
			LastDistortionMID = nullptr;
		}
	}
}

#if WITH_EDITOR
void ACompositingCaptureBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ACompositingCaptureBase, TargetCameraActorPtr))
	{
		// If there is no target camera, remove the last distortion post-process MID from the scene capture
		if (TargetCameraActorPtr == nullptr)
		{
			if (SceneCaptureComponent2D)
			{
				SceneCaptureComponent2D->RemoveBlendable(LastDistortionMID);
			}

			LastDistortionMID = nullptr;

			return;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACompositingCaptureBase, LensComponentPicker))
	{
		if (ACameraActor* TargetCamera = FindTargetCamera())
		{
			ULensComponent* LensComponent = Cast<ULensComponent>(LensComponentPicker.GetComponent(TargetCamera));
			if (LensComponent && LensComponent->GetOwner() != TargetCamera)
			{
				UE_LOGF(LogCompositingCaptureBase, Warning, "Lens Component '%ls' is not attached to the target Camera Actor '%ls' for the CG layer '%ls'."
					, *LensComponent->GetReadableName()
					, *TargetCamera->GetActorLabel(false)
					, *this->GetActorLabel(false));
			}
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void ACompositingCaptureBase::SetApplyDistortion(bool bInApplyDistortion)
{
	bApplyDistortion = bInApplyDistortion;
	UpdateDistortion();
}

void ACompositingCaptureBase::SetLens(ULensComponent* InLens)
{
	if (InLens)
	{
		LensComponentPicker.OtherActor = InLens->GetOwner();
		LensComponentPicker.PathToComponent = InLens->GetPathName(InLens->GetOwner());
	}
}
