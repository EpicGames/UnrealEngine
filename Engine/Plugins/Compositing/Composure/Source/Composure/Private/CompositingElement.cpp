// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElement.h"
#include "Async/TaskGraphInterfaces.h"
#include "CompositingElements/CompElementRenderTargetPool.h"
#include "ComposurePlayerCompositingTarget.h"
#include "ComposurePostProcessingPassProxy.h"
#include "ComposureInternals.h"
#include "CineCameraActor.h"
#include "EngineUtils.h" // for TActorIterator<>
#include "ComposureCustomVersion.h"
#include "CompositingElements/CompositingElementInputs.h" // for UCompositingMediaInput
#include "CompositingElements/CompositingElementTransforms.h" // for UCompositingElementMaterialPass, & UAlphaTransformPass
#include "CompositingElements/CompositingElementOutputs.h" // for UCompositingMediaCaptureOutput
#include "CompositingElements/CompositingElementPassUtils.h" // for NewInstancedSubObj()
#include "UObject/ConstructorHelpers.h"

#if WITH_EDITOR
#include "Editor.h" // for FEditorDelegates::PostPIEStarted
#include "EditorSupport/ICompositingEditor.h"
#endif 

static TAutoConsoleVariable<int32> CVarDisableActiveRendering(
	TEXT("r.Composure.CompositingElements.DisableActiveRendering"),
	0,
	TEXT("Composure compositing elements normally automatically enqueue render commands (both in game and editor). ")
	TEXT("This CVar can be used as a shunt to control perf, where you can shut off all active element rendering. ")
	TEXT("Specific elements can still have their render commands enqueued via an explicit call in game code."));

static TAutoConsoleVariable<int32> CVarUseInternalPassLists(
	TEXT("r.Composure.CompositingElements.Debug.UseInternalPassLists"),
	1,
	TEXT("To manage compositing pass objects that are no longer active, but alive (in the transaction buffer), ")
	TEXT("element objects keep an internal copy of their pass lists, and run off that instead of iterating over the user exposed one. \n")
	TEXT("In case these lists get out of sync, this toggle lets you poll the user set ones directly."));

static TAutoConsoleVariable<int32> CVarReuseIntermediatePassTargets(
	TEXT("r.Composure.CompositingElements.ReuseIntermediatePassTargets"),
	1,
	TEXT("To lighten the Render Target load, set this to release intermediate pass targets back to the pool ")
	TEXT("so they can be used by other, subsequent passes."));

static TAutoConsoleVariable<int32> CVarDisableWhenOpacityIsZero(
	TEXT("r.Composure.CompositingElements.DisableElementWhenOpacityIsZero"),
	1,
	TEXT("When you set and element's opacity to zero, if this is set, we turn off the entire element - as it've you disabled it manually."));


/* CompositingElement_Impl
 *****************************************************************************/

namespace CompositingElement_Impl
{
	template<typename T, typename U>
	static void RefreshInternalPassList(const TArray<T>& PublicList, const TMap<T, U>& ConstructedList, TArray<T>& InternalList);
	
	template<typename T>
	static int32 ClearBlueprintConstructedPasses(TMap<T, ECompPassConstructionType>& ConstructedList);

	template<typename T, typename U>
	static int32 RemovePassesOfType(TArray<T>& PublicList, TMap<T, U>& ConstructedList, TSubclassOf<UCompositingElementPass> PassType);

	template<typename T>
	static void BeginFrameForPasses(const TArray<T>& PassList, bool bCameraCutThisFrame);
	template<typename T>
	static void EndFrameForPasses(const TArray<T>& PassList);

	template<typename T>
	static UTexture* FindLastRenderResult(const TArray<T>& PassList, const FCompositingTextureLookupTable& ResultLookupTable);
}

template<typename T, typename U>
static void CompositingElement_Impl::RefreshInternalPassList(const TArray<T>& PublicList, const TMap<T, U>& ConstructedList, TArray<T>& InternalList)
{
	TArray<T> NewInternalPassList;
	NewInternalPassList.Reserve(PublicList.Num() + ConstructedList.Num());

	auto ParsePassList = [&NewInternalPassList, &InternalList](const TArray<T>& PassList)
	{
		for (const T& CompositingPass : PassList)
		{
			const int32 InternalIndex = InternalList.Find(CompositingPass);
			if (InternalIndex != INDEX_NONE)
			{
				// maintaining order doesn't matter anymore, as we're about to replace the list
				InternalList.RemoveAtSwap(InternalIndex);
			}

			if (CompositingPass)
			{
				NewInternalPassList.Add(CompositingPass);
			}
		}
	};
	ParsePassList(PublicList);

	TArray<T> ConstructedPasses;
	ConstructedList.GenerateKeyArray(ConstructedPasses);
	ParsePassList(ConstructedPasses);

	for (const T& RemovedPass : InternalList)
	{
		if (RemovedPass)
		{
			RemovedPass->Reset();
		}
	}

	InternalList = NewInternalPassList;
}

template<typename T>
static int32 CompositingElement_Impl::ClearBlueprintConstructedPasses(TMap<T, ECompPassConstructionType>& ConstructedList)
{
	int32 RemoveCount = 0;
	for (auto It = ConstructedList.CreateIterator(); It; ++It)
	{
		if (It->Value == ECompPassConstructionType::BlueprintConstructed)
		{
			It.RemoveCurrent();
			++RemoveCount;
		}
	}
	return RemoveCount;
}

template<typename T, typename U>
static int32 CompositingElement_Impl::RemovePassesOfType(TArray<T>& PublicList, TMap<T, U>& ConstructedList, TSubclassOf<UCompositingElementPass> PassType)
{
	int32 RemovedCount = 0;

	for (auto It = ConstructedList.CreateIterator(); It; ++It)
	{
		if (It.Key() && It.Key()->IsA(PassType))
		{
			It.RemoveCurrent();
			++RemovedCount;
		}
	}

	for (int32 PassIndex = PublicList.Num()-1; PassIndex >= 0; --PassIndex)
	{
		const T& Pass = PublicList[PassIndex];
		if (Pass && Pass->IsA(PassType))
		{
			PublicList.RemoveAt(PassIndex);
			++RemovedCount;
		}
	}

	return RemovedCount;
}

template<typename T>
static void CompositingElement_Impl::BeginFrameForPasses(const TArray<T>& PassList, bool bCameraCutThisFrame)
{
	for (const T& Pass : PassList)
	{
		if (Pass)
		{
			Pass->OnFrameBegin(bCameraCutThisFrame);
		}
	}
}

template<typename T>
static void CompositingElement_Impl::EndFrameForPasses(const TArray<T>& PassList)
{
	for (const T& Pass : PassList)
	{
		if (Pass)
		{
			Pass->OnFrameEnd();
		}
	}
}

template<typename T>
static UTexture* CompositingElement_Impl::FindLastRenderResult(const TArray<T>& PassList, const FCompositingTextureLookupTable& ResultLookupTable)
{
	for (int32 PassIndex = PassList.Num() - 1; PassIndex >= 0; --PassIndex)
	{
		const T& Pass = PassList[PassIndex];
		if (Pass && Pass->IsPassEnabled())
		{
			UTexture* OldResult = nullptr;				
			const bool bFound = ResultLookupTable.FindNamedPassResult(Pass->PassName, /*bSearchLinkedTables =*/false, OldResult);

			if (OldResult)
			{
				return OldResult;
			}
		}
	}

	return nullptr;
}

/* ACompositingElement
 *****************************************************************************/

ACompositingElement::ACompositingElement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RenderResolution(1920, 1080)
	, RenderFormat(RTF_RGBA16f)
	, bUseSharedTargetPool(true)
	, FreezeFrameMask(0x00)
#if WITH_EDITORONLY_DATA
	, FreezeFrameController(FreezeFrameMask)
#endif
{
	CompShotIdName = GetFName();

	CompositingTarget = CreateDefaultSubobject<UComposureCompositingTargetComponent>(TEXT("CompositingTarget"));
	PostProcessProxy  = CreateDefaultSubobject<UComposurePostProcessingPassProxy>(TEXT("PostProcessProxy"));
	RootComponent = PostProcessProxy;

#if WITH_EDITOR
	PostProcessProxy->bVisualizeComponent = true;

	COMPOSURE_GET_TEXTURE(Texture, DisabledMsgImage, "Debug/", "T_DisabledElement");
	COMPOSURE_GET_TEXTURE(Texture, EmptyWarnImage, "Debug/", "T_EmptyElement");
	COMPOSURE_GET_TEXTURE(Texture, SuspendedDbgImage, "Debug/", "T_SuspendedElement");
	COMPOSURE_GET_TEXTURE(Texture, CompilerErrImage, "Debug/", "T_CompilerError");

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FEditorDelegates::PostPIEStarted.AddUObject(this, &ACompositingElement::OnPIEStarted);
	}

	DefaultInputType = UMediaTextureCompositingInput::StaticClass();//UMediaBundleCompositingInput::StaticClass();
	DefaultTransformType = UCompositingElementMaterialPass::StaticClass();
	DefaultOutputType = UCompositingMediaCaptureOutput::StaticClass();
#endif
}

void ACompositingElement::SetElementName(const FName NewName)
{
	CompShotIdName = NewName;
#if WITH_EDITOR
	FActorLabelUtilities::RenameExistingActor(this, NewName.ToString());
#endif
}

bool ACompositingElement::AttachAsChildLayer(ACompositingElement* Child)
{
	bool bModified = false;
	if (Child && Child->Parent != this)
	{
		// @TODO-BADGERCOMP: handle shared layers
		if (Child->Parent)
		{
			Child->Parent->Modify();
			Child->Parent->DetatchAsChildLayer(Child);
		}

		ChildLayers.Add(Child);
		Child->Parent = this;
		
		bModified = true;
	}
	else if (!ChildLayers.Contains(Child))
	{
		ChildLayers.Add(Child);
		bModified = true;
	}

	return bModified;	
}

bool ACompositingElement::DetatchAsChildLayer(ACompositingElement* Child)
{
	const bool bModified = ChildLayers.Remove(Child) > 0;
	
	if (ensure(Child->Parent == this))
	{
		Child->Parent = nullptr;
	}

	return bModified;
}

bool ACompositingElement::IsSubElement() const
{
	return (GetElementParent() != nullptr);
}

ACompositingElement* ACompositingElement::GetElementParent() const
{
	if (Parent != nullptr)
	{
		return Parent;
	}
	else if (UChildActorComponent* ChildActorComp = GetParentComponent())
	{
		TArray<USceneComponent*> Parents;
		ChildActorComp->GetParentComponents(Parents);

		AActor* ParentOwner = ChildActorComp->GetOwner();// Parents.Num() > 0) ? Parents[0]->GetOwner() : nullptr;
		for (USceneComponent* ParentComp : Parents)
		{
			if (UChildActorComponent* ChildActorParent = Cast<UChildActorComponent>(ParentComp))
			{
				if (ACompositingElement* CompElementParent = Cast<ACompositingElement>(ChildActorParent->GetChildActor()))
				{
					return CompElementParent;
				}
			}
			ParentOwner = ParentComp->GetOwner();
		}

		if (ACompositingElement* ParentAsElement = Cast<ACompositingElement>(ParentOwner))
		{
			return ParentAsElement;
		}
	}

	return nullptr;
}

const TArray<ACompositingElement*> ACompositingElement::GetChildElements() const
{
	TArray<ACompositingElement*> OutChildElements;

	struct GetChildElements_Impl
	{
		static void FindFirstLevelOfChildActors(USceneComponent* Root, TArray<ACompositingElement*>& ChildrenOut)
		{
			auto SearchChildren = [](USceneComponent* SceneComp, TArray<ACompositingElement*>& InnerChildrenOut)
			{
				TArray<USceneComponent*> Children;
				SceneComp->GetChildrenComponents(/*bIncludeAllDescendants =*/false, Children);

				for (USceneComponent* ChildComp : Children)
				{
					GetChildElements_Impl::FindFirstLevelOfChildActors(ChildComp, InnerChildrenOut);
				}
			};

			if (UChildActorComponent* ChildActorComp = Cast<UChildActorComponent>(Root))
			{
				AActor* ChildActor = ChildActorComp->GetChildActor();
				if (ACompositingElement* AsElement = Cast<ACompositingElement>(ChildActor))
				{
					if (ensure(!ChildrenOut.Contains(AsElement)))
					{
						ChildrenOut.Add(AsElement);
					}
				}
				else
				{
					SearchChildren(ChildActorComp, ChildrenOut);
				}				
			}
			else if (Root)
			{
				SearchChildren(Root, ChildrenOut);
			}
		}
	};
	
	GetChildElements_Impl::FindFirstLevelOfChildActors(GetRootComponent(), OutChildElements);
	
	if (UChildActorComponent* ChildActorComp = GetParentComponent())
	{
		for (USceneComponent* ChildActorChild : ChildActorComp->GetAttachChildren())
		{
			if (ChildActorChild->GetOwner() != this)
			{
				GetChildElements_Impl::FindFirstLevelOfChildActors(ChildActorChild, OutChildElements);
			}
		}
	}

	OutChildElements.Append(ChildLayers);
	return OutChildElements;
}

UCompositingElementPass* ACompositingElement::AddNewPass(FName PassName, TSubclassOf<UCompositingElementPass> PassType, ECompPassConstructionType ConstructedBy)
{
	UCompositingElementPass* NewPass = nullptr;
	if (PassType)
	{
		NewPass = FCompositingElementPassUtils::NewInstancedSubObj<UCompositingElementPass>(/*Outer =*/this, PassType);
		if (NewPass)
		{
			NewPass->PassName = PassName;
#if WITH_EDITOR
			NewPass->ConstructionMethod = ConstructedBy;
#endif	
			if (UCompositingElementInput* InputPass = Cast<UCompositingElementInput>(NewPass))
			{
				UserConstructedInputs.Add(InputPass, ConstructedBy);
				RefreshInternalInputsList();
			}
			else if (UCompositingElementTransform* TransformPass = Cast<UCompositingElementTransform>(NewPass))
			{
				UserConstructedTransforms.Add(TransformPass, ConstructedBy);
				RefreshInternalTransformsList();
			}
			else if (UCompositingElementOutput* OutputPass = Cast<UCompositingElementOutput>(NewPass))
			{
				UserConstructedOutputs.Add(OutputPass, ConstructedBy);
				RefreshInternalOutputsList();
			}
		}
	}
	return NewPass;
}

UCompositingElementInput* ACompositingElement::CreateNewInputPass(FName PassName, TSubclassOf<UCompositingElementInput> InputType)
{
	UCompositingElementInput* NewPass = nullptr;
	if (InputType)
	{
		NewPass = FCompositingElementPassUtils::NewInstancedSubObj<UCompositingElementInput>(this, InputType);
		if (NewPass)
		{
			NewPass->PassName = PassName;
			Inputs.Add(NewPass);
			RefreshInternalInputsList();
		}
	}
	return NewPass;
}


UCompositingElementTransform* ACompositingElement::CreateNewTransformPass(FName PassName, TSubclassOf<UCompositingElementTransform> TransformType)
{
	UCompositingElementTransform* NewPass = nullptr;
	if (TransformType)
	{
		NewPass = FCompositingElementPassUtils::NewInstancedSubObj<UCompositingElementTransform>(this, TransformType);
		if (NewPass)
		{
			NewPass->PassName = PassName;
			TransformPasses.Add(NewPass);
			RefreshInternalTransformsList();
		}
	}
	return NewPass;
}

UCompositingElementOutput* ACompositingElement::CreateNewOutputPass(FName PassName, TSubclassOf<UCompositingElementOutput> OutputType)
{
	UCompositingElementOutput* NewPass = nullptr;
	if (OutputType)
	{
		NewPass = FCompositingElementPassUtils::NewInstancedSubObj<UCompositingElementOutput>(this, OutputType);
		if (NewPass)
		{
			NewPass->PassName = PassName;
			Outputs.Add(NewPass);
			RefreshInternalOutputsList();
		}
	}
	return NewPass;
}


bool ACompositingElement::DeletePass(UCompositingElementPass* PassToDelete)
{	
	bool Output = false;
	if (UCompositingElementInput* InputPass = Cast<UCompositingElementInput>(PassToDelete))
	{
		for (UCompositingElementInput* Pass : Inputs)
		{
			if (Pass && Pass->PassName.IsEqual(PassToDelete->PassName))
			{
				Inputs.RemoveSingle(Pass);
				Output = true;
				RefreshInternalInputsList();
				break;
			}
		}
	}
	else if (UCompositingElementOutput* OutputPass = Cast<UCompositingElementOutput>(PassToDelete))
	{
		for (UCompositingElementOutput* Pass : Outputs)
		{
			if (Pass && Pass->PassName.IsEqual(PassToDelete->PassName))
			{
				Outputs.RemoveSingle(Pass);
				Output = true;
				RefreshInternalOutputsList();
				break;
			}
		}
	}
	else if (UCompositingElementTransform* TransPass = Cast<UCompositingElementTransform>(PassToDelete))
	{
		for (UCompositingElementTransform* Pass : TransformPasses)
		{
			if (Pass && Pass->PassName.IsEqual(PassToDelete->PassName))
			{
				TransformPasses.RemoveSingle(Pass);
				Output = true;
				RefreshInternalTransformsList();
				break;
			}
		}
	}
	return Output;
}


bool ACompositingElement::RemovePass(UCompositingElementPass* ElementPass)
{
	if (UCompositingElementInput* InputPass = Cast<UCompositingElementInput>(ElementPass))
	{
		bool bSuccess = UserConstructedInputs.Remove(InputPass) > 0;
		if (bSuccess)
		{
			RefreshInternalInputsList();
		}
		return bSuccess;
	}
	else if (UCompositingElementTransform* TransformPass = Cast<UCompositingElementTransform>(ElementPass))
	{
		bool bSuccess = UserConstructedTransforms.Remove(TransformPass) > 0;
		if (bSuccess)
		{
			RefreshInternalTransformsList();
		}
		return bSuccess;
	}
	else if (UCompositingElementOutput* OutputPass = Cast<UCompositingElementOutput>(ElementPass))
	{
		bool bSuccess = UserConstructedOutputs.Remove(OutputPass) > 0;
		if (bSuccess)
		{
			RefreshInternalOutputsList();
		}
		return bSuccess;
	}
	return false;
}

int32 ACompositingElement::RemovePassesOfType(TSubclassOf<UCompositingElementPass> PassType)
{	
	if (!PassType)
	{
		return 0;
	}

	int32 RemoveCount = 0;

	if (PassType->IsChildOf<UCompositingElementInput>())
	{
		RemoveCount = CompositingElement_Impl::RemovePassesOfType(Inputs, UserConstructedInputs, PassType);
		if (RemoveCount > 0)
		{
			RefreshInternalInputsList();
		}
	}
	else if (PassType->IsChildOf<UCompositingElementTransform>())
	{
		RemoveCount = CompositingElement_Impl::RemovePassesOfType(TransformPasses, UserConstructedTransforms, PassType);
		if (RemoveCount > 0)
		{
			RefreshInternalTransformsList();
		}
	}
	else if (PassType->IsChildOf<UCompositingElementOutput>())
	{
		RemoveCount = CompositingElement_Impl::RemovePassesOfType(Outputs, UserConstructedOutputs, PassType);
		if (RemoveCount > 0)
		{
			RefreshInternalOutputsList();
		}
	}

	return RemoveCount;
}

void ACompositingElement::SetOpacity(const float NewOpacity)
{
	if (OutputOpacity != NewOpacity)
	{
		if (NewOpacity <= 0.f && OutputOpacity > 0.f && !!CVarDisableWhenOpacityIsZero.GetValueOnAnyThread())
		{
			OnDisabled();
		}
		OutputOpacity = NewOpacity;
	}
}

UTextureRenderTarget2D* ACompositingElement::RequestNamedRenderTarget(const FName ReferenceName, const float RenderPercentage, ETargetUsageFlags UsageTag)
{
	if (!ensure((FreezeFrameMask & (int32)UsageTag) == 0x00))
	{
		UE_LOGF(Composure, Warning, "Requesting a render target for usage that is currently freeze-framed - everything should be static while 'frozen'.");
	}

	const FSharedTargetPoolPtr& RenderTargetPoolPtr = GetRenderTargetPool();

	UTexture* PreExistingResult = nullptr;
	const bool bPreExists = PassResultsTable.FindNamedPassResult(ReferenceName, PreExistingResult);
	UTextureRenderTarget2D* PreExistingTarget = Cast<UTextureRenderTarget2D>(PreExistingResult);
	if (bPreExists && PreExistingTarget)
	{
		const int32 PreExistingUsageTags = RenderTargetPoolPtr->FindAssignedUsageTags(PreExistingTarget);
		if ((PreExistingUsageTags & (int32)ETargetUsageFlags::USAGE_Persistent) != 0x00)
		{
			UE_LOGF(Composure, Warning, "Requesting a new render target using the name of one that is already in use - and persistent! Returning that to you instead.");
			return PreExistingTarget;
		}
	}

	const FIntPoint TargetResolution = GetRenderResolution() * RenderPercentage;
	UTextureRenderTarget2D* NewTarget = RenderTargetPoolPtr->AssignTarget(this, TargetResolution, RenderFormat, (int32)UsageTag);

	if (NewTarget && !ReferenceName.IsNone())
	{
		PassResultsTable.RegisterPassResult(ReferenceName, NewTarget, (int32)UsageTag);
	}
	return NewTarget;
}

bool ACompositingElement::ReleaseOwnedTarget(UTextureRenderTarget2D* OwnedTarget)
{
	if (OwnedTarget && RenderTargetPool.IsValid())
	{
		const int32 UsageTags = RenderTargetPool->FindAssignedUsageTags(OwnedTarget);
		if (ensure((UsageTags & FreezeFrameMask) == 0x00))
		{
			bool bSuccess = RenderTargetPool->ReleaseTarget(OwnedTarget);
			if (bSuccess && (UsageTags & (int32)ETargetUsageFlags::USAGE_Persistent) != 0)
			{
				PassResultsTable.Remove(OwnedTarget);
			}
			return bSuccess;
		}
		else 
		{
			UE_LOGF(Composure, Error, "Blocked an attempt to release a render target that is currently freeze-framed - everything should be static while 'frozen'.");
		}
	}
	return false;
}

UTexture* ACompositingElement::RenderCompositingMaterial(FCompositingMaterial& CompMaterial, float RenderScale, FName ResultLookupName, ETargetUsageFlags UsageTag)
{
	const bool bFreezeRender = (FreezeFrameMask & (int32)UsageTag) != 0x00;
	if (bFreezeRender)
	{
		return FindNamedRenderResult(ResultLookupName, /*bSearchSubElements =*/false);
	}
	else
	{
		UTextureRenderTarget2D* RenderTarget = RequestNamedRenderTarget(ResultLookupName, RenderScale, UsageTag);
		return RenderCompositingMaterialToTarget(CompMaterial, RenderTarget, ResultLookupName);
	}
}

UTextureRenderTarget2D* ACompositingElement::RenderCompositingMaterialToTarget(FCompositingMaterial& CompMaterial, UTextureRenderTarget2D* RenderTarget, FName ResultLookupName)
{
	UTextureRenderTarget2D* Result = RenderTarget;

	const bool bFreezeRender = RenderTargetPool.IsValid() && ((RenderTargetPool->FindAssignedUsageTags(RenderTarget) & (int32)FreezeFrameMask) != 0x00);
	if (bFreezeRender)
	{
		Result = Cast<UTextureRenderTarget2D>(FindNamedRenderResult(ResultLookupName, /*bSearchSubElements =*/false));
	}	
	else if (CompMaterial.ApplyParamOverrides(&PassResultsTable) && RenderTarget)
	{
		CompMaterial.RenderToRenderTarget(/*WorldContext =*/this, RenderTarget);
		if (!ResultLookupName.IsNone())
		{
			RegisterPassResult(ResultLookupName, RenderTarget, /*bSetAsLatestRenderResult =*/true);
		}
	}
	return Result;
}

void ACompositingElement::RegisterPassResult(FName ReferenceName, UTexture* PassResult, bool bSetAsLatestRenderResult)
{
	RegisterTaggedPassResult(ReferenceName, PassResult);
	if (bSetAsLatestRenderResult)
	{
		UpdateFinalRenderResult(PassResult);
	}
}

ACameraActor* ACompositingElement::FindTargetCamera() const
{
	if (CameraSource == ESceneCameraLinkType::Override)
	{
		return TargetCameraActorPtr.Get();
	}
	else if (Parent)
	{
		return Parent->FindTargetCamera();
	}
	else if (!TargetCameraActorPtr.IsValid())
	{
		for (TActorIterator<ACineCameraActor> CamActorIt(GetWorld()); CamActorIt; ++CamActorIt)
		{
			ACineCameraActor* FoundActor = *CamActorIt;
			if (IsValid(FoundActor))
			{
				return FoundActor;
			}
		}

		for (TActorIterator<ACameraActor> CamActorIt(GetWorld()); CamActorIt; ++CamActorIt)
		{
			ACameraActor* FoundActor = *CamActorIt;
			if (IsValid(FoundActor))
			{
				return FoundActor;
			}
		}
	}
	return TargetCameraActorPtr.Get();
}

void ACompositingElement::SetTargetCamera(ACameraActor* NewCameraActor)
{
	if (CameraSource == ESceneCameraLinkType::Override)
	{
		//Override mode has the highest priority in using the camera. Should check before going to the parent. 
		TargetCameraActorPtr = NewCameraActor;
	}
	else if (Parent)
	{
		Parent->SetTargetCamera(NewCameraActor);
	}
	else 
	{
		//Base case for the recursive calls. Current element does not have parent and is not in override mode.
		TargetCameraActorPtr = NewCameraActor;
	}
}


UCompositingElementInput* ACompositingElement::FindInputPass(TSubclassOf<UCompositingElementInput> InputType, UTexture*& PassResult, FName OptionalPassName)
{
	UCompositingElementInput* FoundInput = nullptr;
	for (UCompositingElementInput* Input : GetInternalInputsList())
	{
		if (Input && Input->IsA(InputType))
		{
			if (OptionalPassName.IsNone() || Input->PassName == OptionalPassName)
			{
				FoundInput = Input;
				PassResult = FindNamedRenderResult(Input->PassName);
				break;
			}
		}
	}

	return FoundInput;
}

UCompositingElementTransform* ACompositingElement::FindTransformPass(TSubclassOf<UCompositingElementTransform> TransformType, UTexture*& PassResult, FName OptionalPassName)
{
	UCompositingElementTransform* FoundTransform = nullptr;
	for (UCompositingElementTransform* Transform : GetInternalTransformsList())
	{
		if (Transform && Transform->IsA(TransformType))
		{
			if (OptionalPassName.IsNone() || Transform->PassName == OptionalPassName)
			{
				FoundTransform = Transform;
				PassResult = FindNamedRenderResult(Transform->PassName);
				break;
			}
		}
	}

	return FoundTransform;
}

UCompositingElementOutput* ACompositingElement::FindOutputPass(TSubclassOf<UCompositingElementOutput> OutputType, FName OptionalPassName)
{
	UCompositingElementOutput* FoundOutput = nullptr;
	for (UCompositingElementOutput* Output : GetInternalOutputsList())
	{
		if (Output && Output->IsA(OutputType))
		{
			if (OptionalPassName.IsNone() || Output->PassName == OptionalPassName)
			{
				FoundOutput = Output;
				break;
			}
		}
	}

	return FoundOutput;
}

UCompositingElementInput* ACompositingElement::AddNewInputPass(FName PassName, TSubclassOf<UCompositingElementInput> InputType)
{
	return Cast<UCompositingElementInput>(AddNewPass(PassName, InputType, ECompPassConstructionType::BlueprintConstructed));
}

UCompositingElementTransform* ACompositingElement::AddNewTransformPass(FName PassName, TSubclassOf<UCompositingElementTransform> TransformType)
{
	return Cast<UCompositingElementTransform>(AddNewPass(PassName, TransformType, ECompPassConstructionType::BlueprintConstructed));
}

UCompositingElementOutput* ACompositingElement::AddNewOutputPass(FName PassName, TSubclassOf<UCompositingElementOutput> OutputType)
{
	return Cast<UCompositingElementOutput>(AddNewPass(PassName, OutputType, ECompPassConstructionType::BlueprintConstructed));
}

UTexture* ACompositingElement::GetLatestRenderResult() const
{
	if (CompositingTarget)
	{
		return CompositingTarget->GetDisplayTexture();
	}
	return nullptr;
}

FIntPoint ACompositingElement::GetRenderResolution() const
{
	if (ResolutionSource == EInheritedSourceType::Override || !Parent)
	{
		return RenderResolution;
	}
	return Parent->GetRenderResolution();
}

UTexture* ACompositingElement::FindNamedRenderResult(FName PassName, bool bSearchSubElements)
{
	UTexture* FoundResult = nullptr;
	PassResultsTable.FindNamedPassResult(PassName, bSearchSubElements, FoundResult);

	return FoundResult;
}

UTexture* ACompositingElement::RenderCompElement_Implementation(bool /*bCameraCutThisFrame*/)
{
	FInheritedTargetPool PassTargetPool(/*Owner =*/this, GetRenderResolution(), RenderFormat, GetRenderTargetPool(), (int32)ETargetUsageFlags::USAGE_Transform);
	ApplyTransforms(PassTargetPool);

	return GetLatestRenderResult();
}

void ACompositingElement::BeginDestroy()
{
	Super::BeginDestroy();

	// return render targets to the pool before breaking our link with the pool
	if (RenderTargetPool.IsValid())
	{
		RenderTargetPool->ReleaseAssignedTargets(this);
		RenderTargetPool.Reset();
	}
	FrameReset();

#if WITH_EDITOR
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
#endif
}

#if WITH_EDITOR
void ACompositingElement::RerunConstructionScripts()
{
	int32 ClearCount = CompositingElement_Impl::ClearBlueprintConstructedPasses(UserConstructedInputs);
	ClearCount += CompositingElement_Impl::ClearBlueprintConstructedPasses(UserConstructedTransforms);
	ClearCount += CompositingElement_Impl::ClearBlueprintConstructedPasses(UserConstructedOutputs);

	Super::RerunConstructionScripts();

	//if (ClearCount > 0)
	{
		RefreshAllInternalPassLists();
	}

	if (ICompositingEditor* CompositingEditor = ICompositingEditor::Get())
	{
		CompositingEditor->RequestRedraw();
	}
}
#endif

void ACompositingElement::PostInitProperties()
{
	Super::PostInitProperties();
	RefreshAllInternalPassLists();
}

void ACompositingElement::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FComposureCustomVersion::GUID);

	const int32 ComposureVer = Ar.CustomVer(FComposureCustomVersion::GUID);
	PostSerializeCompatUpgrade(ComposureVer);
}

void ACompositingElement::PostLoad()
{
	Super::PostLoad();

	const int32 ComposureVer = GetLinkerCustomVersion(FComposureCustomVersion::GUID);
	PostLoadCompatUpgrade(ComposureVer);

#if WITH_EDITORONLY_DATA
	// Re-assign the value of deprecated pointer to the new one.
	// This will continue until user re-saves the level.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (TargetCameraActor_DEPRECATED.IsValid())
	{
		TargetCameraActorPtr = TargetCameraActor_DEPRECATED.Get();
		TargetCameraActor_DEPRECATED = nullptr;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	// Note that RefreshAllInternalPassLists is not called here because the refresh can call Reset, which is not allowed in PostLoad.
}

void ACompositingElement::BeginPlay()
{
	Super::BeginPlay();
	RefreshAllInternalPassLists();
}

void ACompositingElement::SetAutoRun(bool bNewAutoRunVal)
{
	if (bAutoRun != bNewAutoRunVal)
	{
		bAutoRun = bNewAutoRunVal;
		if (!bAutoRun && !IsActivelyRunning())
		{
			OnDisabled();
		}
	}
}

void ACompositingElement::EnqueueRendering_Implementation(bool bCameraCutThisFrame)
{
#if WITH_EDITOR
	if (ICompositingEditor* CompositingEditor = ICompositingEditor::Get())
	{
		if (CompositingEditor->DeferCompositingDraw(this))
		{
			return;
		}
	}
#endif

	FrameReset();
	BeginFrameForAllPasses(bCameraCutThisFrame);

	UTexture* RenderResult = GetLatestRenderResult();
	{
		GenerateInputs();

		const bool bFreezeBlueprintCall = (FreezeFrameMask & (int32)ETargetUsageFlags::USAGE_Transform) != 0x00;
		if (!bFreezeBlueprintCall)
		{
			RenderResult = RenderCompElement(bCameraCutThisFrame);
		}
		else
		{
			RenderResult = CompositingElement_Impl::FindLastRenderResult(GetInternalTransformsList(), PassResultsTable);
			if (!RenderResult)
			{
				RenderResult = GetLatestRenderResult();
			}
		}
	}

	if (OutputOpacity < 1.0f && RenderResult)
	{
		if (InternalAlphaPass == nullptr)
		{
			InternalAlphaPass = FCompositingElementPassUtils::NewInstancedSubObj<UAlphaTransformPass>(/*Outer =*/this);
		}
		InternalAlphaPass->AlphaScale = OutputOpacity;

		FInheritedTargetPool AlphaPassTargetPool(/*Owner =*/this, GetRenderResolution(), RenderFormat, GetRenderTargetPool(), (int32)ETargetUsageFlags::USAGE_Transform);

		UTexture* Result = InternalAlphaPass->ApplyTransform(RenderResult, &PassResultsTable, PostProcessProxy, FindTargetCamera(), AlphaPassTargetPool);
		if (Result && Result != RenderResult)
		{
			RenderResult = Result;
			UpdateFinalRenderResult(Result);
			IncIntermediateTrackingTag();
		}
	}
	else if (InternalAlphaPass)
	{
		InternalAlphaPass->Reset();
		InternalAlphaPass = nullptr;
	}

#if WITH_EDITOR
	if (RenderResult == nullptr)
	{
		SetDebugDisplayImage(EmptyWarnImage);
	}
	else
#endif
	{
		UpdateFinalRenderResult(RenderResult);
	}

	OnFinalPassRendered.Broadcast(this, RenderResult);
	OnFinalPassRendered_BP.Broadcast(this, RenderResult);

	//
	{
		FInheritedTargetPool OutputTargetPool(/*Owner =*/this, GetRenderResolution(), RenderFormat, GetRenderTargetPool(), (int32)ETargetUsageFlags::USAGE_Output);
		RelayOutputs(OutputTargetPool);
	}
	
	EndFrameForAllPasses();
}

bool ACompositingElement::IsActivelyRunning_Implementation() const
{
	return AComposurePipelineBaseActor::IsActivelyRunning_Implementation() && !CVarDisableActiveRendering.GetValueOnGameThread() && 
		(!CVarDisableWhenOpacityIsZero.GetValueOnGameThread() || OutputOpacity > 0.f);
}

int32 ACompositingElement::GetRenderPriority() const
{
	if (Parent)
	{
		return Parent->GetRenderPriority() + 1;
	}
	return FCompElementRenderTargetPool::ExtensionPriority + 1;
}

void ACompositingElement::FrameReset()
{
	if (RenderTargetPool.IsValid())
	{
		RenderTargetPool->ReleaseAssignedTargets(this, FreezeFrameMask | (int32)ETargetUsageFlags::USAGE_Persistent);
	}

	ResetResultsLookupTable();

#if WITH_EDITOR
	bUsingDebugDisplayImage = false;

	EditorPreviewImage = nullptr;
	ColorPickerDisplayImage = nullptr;
	ColorPickerTarget = nullptr;
#endif

	if (CompositingTarget)
	{
		CompositingTarget->SetDisplayTexture(nullptr);
	}
}

void ACompositingElement::OnDisabled()
{
	for (auto ResultIt : PassResultsTable)
	{
		ResultIt.Value.Texture = nullptr;
	}

#if WITH_EDITOR
	SetDebugDisplayImage(DisabledMsgImage);

	if (ICompositingEditor* CompositingEditor = ICompositingEditor::Get())
	{
		CompositingEditor->RequestRedraw();
	}
#endif

	FreezeFrameMask = 0x00;
	if (RenderTargetPool.IsValid())
	{
		RenderTargetPool->ReleaseAssignedTargets(this, (int32)ETargetUsageFlags::USAGE_Persistent);
	}
}

void ACompositingElement::RefreshAllInternalPassLists()
{
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		RefreshInternalInputsList();
		RefreshInternalTransformsList();
		RefreshInternalOutputsList();
	}
}

void ACompositingElement::RefreshInternalInputsList()
{
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		CompositingElement_Impl::RefreshInternalPassList(Inputs, UserConstructedInputs, InternalInputs);
	}
}

void ACompositingElement::RefreshInternalTransformsList()
{
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		CompositingElement_Impl::RefreshInternalPassList(TransformPasses, UserConstructedTransforms, InternalTransformPasses);
	}
}

void ACompositingElement::RefreshInternalOutputsList()
{
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		CompositingElement_Impl::RefreshInternalPassList(Outputs, UserConstructedOutputs, InternalOutputs);
	}
}

const decltype(ACompositingElement::Inputs)& ACompositingElement::GetInternalInputsList() const
{
	if (CVarUseInternalPassLists.GetValueOnGameThread() && !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		return InternalInputs;
	}
	else
	{
		return Inputs;
	}
}

const decltype(ACompositingElement::TransformPasses)& ACompositingElement::GetInternalTransformsList() const
{
	if (CVarUseInternalPassLists.GetValueOnGameThread() && !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		return InternalTransformPasses;
	}
	else
	{
		return TransformPasses;
	}
}

const decltype(ACompositingElement::Outputs)& ACompositingElement::GetInternalOutputsList() const
{
	if (CVarUseInternalPassLists.GetValueOnGameThread() && !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		return InternalOutputs;
	}
	else
	{
		return Outputs;
	}
}

void ACompositingElement::BeginFrameForAllPasses(bool bCameraCutThisFrame)
{
	CompositingElement_Impl::BeginFrameForPasses(GetInternalInputsList(), bCameraCutThisFrame);
	CompositingElement_Impl::BeginFrameForPasses(GetInternalTransformsList(), bCameraCutThisFrame);
	CompositingElement_Impl::BeginFrameForPasses(GetInternalOutputsList(), bCameraCutThisFrame);
}

void ACompositingElement::GenerateInputs()
{
	if ((FreezeFrameMask & (int32)ETargetUsageFlags::USAGE_Input) == 0)
	{
		FInheritedTargetPool SharedTargetPool(/*Owner =*/this, GetRenderResolution(), RenderFormat, GetRenderTargetPool(), (int32)ETargetUsageFlags::USAGE_Input);

		for (UCompositingElementInput* Input : GetInternalInputsList())
		{
			if (Input)
			{
				const bool bIsIntermediate = (Input->bIntermediate || Input->PassName.IsNone()) && !!CVarReuseIntermediatePassTargets.GetValueOnGameThread();
				const ETargetUsageFlags UsageTags = bIsIntermediate ? (ETargetUsageFlags::USAGE_Input | NextIntermediateTrackingTag) : ETargetUsageFlags::USAGE_Input;

				UTexture* Result = nullptr;
				if (Input->IsPassEnabled())
				{
					FScopedTargetPoolTagAddendum IntermediateTagger((int32)UsageTags, SharedTargetPool);
					Result = Input->GenerateInput(SharedTargetPool);
				}

				RegisterTaggedPassResult(Input->PassName, Result, UsageTags);
				UpdateFinalRenderResult(Result);
			}
			
		}

		// We don't increment the intermediate tracking tag in the Inputs loop, because ALL inputs
		// should be available to the first TransformPass.
		IncIntermediateTrackingTag();
		// @TODO: What if there are no transform passes? We should release all but the last Input.
	}
	else
	{
		UTexture* OldResult = CompositingElement_Impl::FindLastRenderResult(GetInternalInputsList(), PassResultsTable);
		if (OldResult)
		{
			UpdateFinalRenderResult(OldResult);
		}
	}
}

void ACompositingElement::ApplyTransforms(FInheritedTargetPool& SharedTargetPool)
{
	if ((FreezeFrameMask & (int32)ETargetUsageFlags::USAGE_Transform) == 0)
	{
		ACameraActor* TargetCam = FindTargetCamera();

		UTexture* PreviousPass = GetLatestRenderResult();
		for (UCompositingElementTransform* TransformPass : GetInternalTransformsList())
		{
			if (TransformPass)
			{
				const bool bIsIntermediate = (TransformPass->bIntermediate || TransformPass->PassName.IsNone()) && !!CVarReuseIntermediatePassTargets.GetValueOnGameThread();
				const ETargetUsageFlags UsageTags = bIsIntermediate ? (ETargetUsageFlags::USAGE_Transform | NextIntermediateTrackingTag) : ETargetUsageFlags::USAGE_Transform;

				UTexture* Result = nullptr;
				if (TransformPass->IsPassEnabled())
				{
					FScopedTargetPoolTagAddendum IntermediateTagger((int32)UsageTags, SharedTargetPool);
					Result = TransformPass->ApplyTransform(PreviousPass, &PassResultsTable, PostProcessProxy, TargetCam, SharedTargetPool);
				}
				
				RegisterTaggedPassResult(TransformPass->PassName, Result, UsageTags);
				
				if (Result && Result != PreviousPass)
				{
					PreviousPass = Result;
					UpdateFinalRenderResult(Result);

					IncIntermediateTrackingTag();
				}
			}
		}
	}
	else
	{
		UTexture* OldResult = CompositingElement_Impl::FindLastRenderResult(GetInternalTransformsList(), PassResultsTable);
		if (OldResult)
		{
			UpdateFinalRenderResult(OldResult);
		}
	}
}

void ACompositingElement::RelayOutputs(const FInheritedTargetPool& SharedTargetPool)
{
	UTexture* ElementRenderResult = GetLatestRenderResult();

#if WITH_EDITOR
	EditorPreviewImage = ElementRenderResult;

	if (CompositingTarget)
	{
		CompositingTarget->SetUseImplicitGammaForPreview(true);
	}

	UCompositingElementTransform* PreviewTransforPass = GetPreviewPass();
	if (PreviewTransforPass && PreviewTransforPass->IsPassEnabled() && IsPreviewing())
	{
		ACameraActor* TargetCamera = FindTargetCamera();
		if (ColorPickerDisplayImage)
		{
			ColorPickerDisplayImage = PreviewTransforPass->ApplyTransform(ColorPickerDisplayImage, &PassResultsTable, PostProcessProxy, TargetCamera, SharedTargetPool);
		}

		if (EditorPreviewImage && !bUsingDebugDisplayImage)
		{
			EditorPreviewImage = PreviewTransforPass->ApplyTransform(EditorPreviewImage, &PassResultsTable, PostProcessProxy, TargetCamera, SharedTargetPool);
			if (CompositingTarget)
			{
				CompositingTarget->SetDisplayTexture(EditorPreviewImage);
				CompositingTarget->SetUseImplicitGammaForPreview(false);
			}
		}
	}
#endif // WITH_EDITOR

	if ((FreezeFrameMask & (int32)ETargetUsageFlags::USAGE_Output) == 0)
	{
		for (UCompositingElementOutput* Output : GetInternalOutputsList())
		{
			if (Output && Output->IsPassEnabled())
			{
				Output->RelayOutput(ElementRenderResult, PostProcessProxy, SharedTargetPool);
			}
		}
	}
}

void ACompositingElement::EndFrameForAllPasses()
{
	CompositingElement_Impl::EndFrameForPasses(GetInternalInputsList());
	CompositingElement_Impl::EndFrameForPasses(GetInternalTransformsList());
	CompositingElement_Impl::EndFrameForPasses(GetInternalOutputsList());
}

void ACompositingElement::UpdateFinalRenderResult(UTexture* RenderResult)
{
	if (RenderResult != nullptr)
	{
		if (CompositingTarget)
		{
			CompositingTarget->SetDisplayTexture(RenderResult);
		}
		PassResultsTable.SetMostRecentResult(RenderResult);
	}
}

const FSharedTargetPoolPtr& ACompositingElement::GetRenderTargetPool()
{
	if (!RenderTargetPool.IsValid())
	{
		if (bUseSharedTargetPool)
		{
			RenderTargetPool = FCompElementRenderTargetPool::GetSharedInstance();
		}
		else
		{
			RenderTargetPool = MakeShareable(new FCompElementRenderTargetPool(this));
		}
	}
	return RenderTargetPool;
}

void ACompositingElement::RegisterTaggedPassResult(FName ReferenceName, UTexture* PassResult, ETargetUsageFlags UsageFlags)
{
	if (!ReferenceName.IsNone())
	{
		int32 UsageMask = (int32)UsageFlags;
		if (UsageFlags == ETargetUsageFlags::USAGE_None && RenderTargetPool.IsValid())
		{
			UsageMask = RenderTargetPool->FindAssignedUsageTags(Cast<UTextureRenderTarget2D>(PassResult));
		}

		if (!!(PassResultsTable.FindUsageTags(ReferenceName) & (int32)ETargetUsageFlags::USAGE_Persistent))
		{
			UTexture* PreExistingResult = nullptr;
			PassResultsTable.FindNamedPassResult(ReferenceName, /*bSearchLinkedTables =*/false, PreExistingResult);
			UE_CLOGF(PreExistingResult != PassResult, Composure, Error, "Attempting to register a pass result over a pre-existing persistent one. Blocking this action (release the persistent target first).");
		}
		else
		{
			PassResultsTable.RegisterPassResult(ReferenceName, PassResult, UsageMask);
		}		
	}

	OnTransformPassRendered.Broadcast(this, PassResult, ReferenceName);
	OnTransformPassRendered_BP.Broadcast(this, PassResult, ReferenceName);
}

void ACompositingElement::ResetResultsLookupTable(bool bKeepPassResults)
{
	if (!bKeepPassResults)
	{
		PassResultsTable.Empty(FreezeFrameMask | (int32)ETargetUsageFlags::USAGE_Persistent);
	}

	PassResultsTable.ClearLinkedSearchTables();

	for (ACompositingElement* Child : ChildLayers)
	{
		if (Child)
		{
			PassResultsTable.LinkNestedSearchTable(Child->GetCompElementName(), &Child->PassResultsTable);
		}
	}
}

void ACompositingElement::IncIntermediateTrackingTag()
{
	static const ETargetUsageFlags IntermediateTagMask = ETargetUsageFlags::USAGE_Intermediate0 | ETargetUsageFlags::USAGE_Intermediate1;
	NextIntermediateTrackingTag = (~NextIntermediateTrackingTag) & IntermediateTagMask;

	PassResultsTable.ClearTaggedEntries((int32)NextIntermediateTrackingTag);

	if (RenderTargetPool.IsValid())
	{
		RenderTargetPool->ReleaseTaggedTargets((int32)NextIntermediateTrackingTag, this);
	}
}


void ACompositingElement::SetAutoRunChildrenAndSelf(bool bAutoRunChildAndSelf)
{
	TArray<ACompositingElement*> FoundChildren = GetChildElements();
	for (ACompositingElement* Child : FoundChildren)
	{
		if (Child)
		{
			Child->SetAutoRunChildrenAndSelf(bAutoRunChildAndSelf);
		}
	}

	//Enable/Disable the auto running rendering of current composure element
	SetAutoRun(bAutoRunChildAndSelf);
	bAutoRunChildElementsAndSelf = bAutoRunChildAndSelf;
}


