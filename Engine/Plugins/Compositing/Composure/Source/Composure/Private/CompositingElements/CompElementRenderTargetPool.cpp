// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElements/CompElementRenderTargetPool.h"
#include "Kismet/KismetRenderingLibrary.h" // for ReleaseRenderTarget2D()
#include "ComposureInternals.h" // for the 'Composure' log category
#include "UObject/Package.h"

namespace ElementRenderTargetPool_Impl
{
	static void LogSharedTargetCount(FOutputDevice& OutputDevice)
	{
		FSharedTargetPoolRef SharedTargetPool = FCompElementRenderTargetPool::GetSharedInstance();
		OutputDevice.Logf(TEXT("Number of compositing render targets currently in use: %d"), SharedTargetPool->GetTargetCount());
	}
}

static TAutoConsoleVariable<int32> CVarRenderTargetLimit(
	TEXT("r.Composure.CompositingElements.Debug.RenderTargetPoolLimit"),
	0,
	TEXT("When greater than zero, this will limit how many render targets are allocated in a single frame. ")
	TEXT("Helpful for catching target leaks (when you know the expected target count)."));

static TAutoConsoleVariable<int32> CVarBreakOnTargetAlloc(
	TEXT("r.Composure.CompositingElements.Debug.BreakOnNewTargetAllocations"),
	0,
	TEXT("When enabled this will trigger a ensure (a soft assert) whenever a new RenderTarget is allocated for the compositing system. ")
	TEXT("Helpful for catching target leaks - enable when you're not in the middle of modifying your pipeline."));

static TAutoConsoleVariable<int32> CVarBreakOnTargetFlush(
	TEXT("r.Composure.CompositingElements.Debug.BreakOnFlushedTarget"),
	0,
	TEXT("When enabled this will trigger a ensure (a soft assert) whenever a target from the pool is flushed. ")
	TEXT("Helpful for catching mismanaged target usage - when you're not altering target size/formats or deleting elements/passes, your pool should not have to flush."));

static TAutoConsoleVariable<int32> CVarAutoFlushUnusedTargets(
	TEXT("r.Composure.CompositingElements.Editor.AutoFlushStaleTargets"),
	1,
	TEXT("In editor, you can alter the target size and render format, or delete passes/elements. ")
	TEXT("This may leave render resources pooled, but never reclaimed. Auto-Flushing returns those resources. ")
	TEXT("For values greater than zero, the pooling system will wait that number of frames before a target is considered 'stale'."));

static FAutoConsoleCommandWithOutputDevice FLogSharedTargetCountCommand(
	TEXT("r.Composure.CompositingElements.Debug.LogSharedTargetsCount"),
	TEXT("Dumps the count of all target currently allocated for the shared target compositing target pool."),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic(&ElementRenderTargetPool_Impl::LogSharedTargetCount));

/* FCompElementRenderTargetPool
  *****************************************************************************/

int32 FCompElementRenderTargetPool::ExtensionPriority = 0;
FWeakTargetPoolPtr FCompElementRenderTargetPool::SharedInstance;

//------------------------------------------------------------------------------
FSharedTargetPoolRef FCompElementRenderTargetPool::GetSharedInstance()
{
	if (SharedInstance.IsValid())
	{
		return SharedInstance.Pin().ToSharedRef();
	}
	else
	{
		FSharedTargetPoolRef NewPool = MakeShareable(new FCompElementRenderTargetPool(/*Outer =*/GetTransientPackage()));
		SharedInstance = NewPool;
		return NewPool;
	}
}

//------------------------------------------------------------------------------
FCompElementRenderTargetPool::FCompElementRenderTargetPool(UObject* Outer)
	: PoolOwner(Outer)
{}

//------------------------------------------------------------------------------
FCompElementRenderTargetPool::~FCompElementRenderTargetPool()
{
	FlushAllTargets();
}

//------------------------------------------------------------------------------
UTextureRenderTarget2D* FCompElementRenderTargetPool::AssignTarget(UObject* Owner, FIntPoint Dimensions, ETextureRenderTargetFormat Format, const int32 UsageTags)
{
	UTextureRenderTarget2D* AssignedTarget = nullptr;

	int32 MaxResolution = static_cast<int32>(GetMax2DTextureDimension());

	bool bMaxSizeExceededWarning = false;

	// Scale down dimensions if they exceed the maximum allowed resolution, while maintaining the aspect ratio.
	if (Dimensions.GetMax() > MaxResolution)
	{
		float Scale = 1.0f;
		if (Dimensions.X >= Dimensions.Y)
		{
			Scale = MaxResolution / static_cast<float>(Dimensions.X);
		}
		else
		{
			Scale = MaxResolution / static_cast<float>(Dimensions.Y);
		}
		Dimensions.X = static_cast<int32>(Scale * Dimensions.X);
		Dimensions.Y = static_cast<int32>(Scale * Dimensions.Y);
		
		bMaxSizeExceededWarning = true;
	}

	FRenderTargetDesc TargetDesc = { Dimensions, Format };
	FPooledTarget* PooledTargetPtr = RenderTargetPool.Find(TargetDesc);

	if (PooledTargetPtr)
	{
		if (ensure(CVarRenderTargetLimit.GetValueOnGameThread() <= 0 || AssignedTargets.Num() < CVarRenderTargetLimit.GetValueOnGameThread()))
		{
			AssignedTarget = *PooledTargetPtr;
			RenderTargetPool.RemoveSingle(TargetDesc, AssignedTarget);
		}
	}
	else if (Dimensions.X > 0 && Dimensions.Y > 0)
	{
		if (ensure(CVarRenderTargetLimit.GetValueOnGameThread() <= 0 || GetTargetCount() < CVarRenderTargetLimit.GetValueOnGameThread()))
		{
			ensureAlways(!CVarBreakOnTargetAlloc.GetValueOnGameThread());

			if (bMaxSizeExceededWarning)
			{
				UE_LOGF(Composure, Warning, "Requested texture exceeds the maximum allowed dimensions, resized to: %dx%d", Dimensions.X, Dimensions.Y);
			}

			// don't use the passed in Owner as the outer, since this will be repooled and shared
			UObject* Outer = ensure(PoolOwner.IsValid()) ? PoolOwner.Get() : GetTransientPackage();
			// emulate UKismetRenderingLibrary::CreateRenderTarget2D()
			UTextureRenderTarget2D* NewRenderTarget2D = NewObject<UTextureRenderTarget2D>(Outer);
			{
				NewRenderTarget2D->RenderTargetFormat = Format;
				NewRenderTarget2D->InitAutoFormat(Dimensions.X, Dimensions.Y);
				NewRenderTarget2D->UpdateResourceImmediate(true);
			}
			AssignedTarget = NewRenderTarget2D;
		}
	}

	if (AssignedTarget)
	{
		AssignedTargets.Add(AssignedTarget, FTargetAssignee(Owner, UsageTags));
	}
	return AssignedTarget;
}

//------------------------------------------------------------------------------
bool FCompElementRenderTargetPool::ReleaseTarget(UTextureRenderTarget2D* RenderTarget)
{
	bool bSuccess = false;
	if (AssignedTargets.Remove(RenderTarget) > 0)
	{
		FRenderTargetDesc TargetDesc = { FIntPoint(RenderTarget->SizeX, RenderTarget->SizeY), RenderTarget->RenderTargetFormat };
		RenderTargetPool.Add(TargetDesc, RenderTarget);

		bSuccess = true;
	}
	else
	{
		UE_LOGF(Composure, Warning, "Attempting to release a render target that doesn't belong to this pool - possible leak?");
	}
	return bSuccess;
}

//------------------------------------------------------------------------------
void FCompElementRenderTargetPool::ReleaseAssignedTargets(UObject* Owner, const int32 KeepTags)
{
	TArray<UTextureRenderTarget2D*> TargetsToRelease;
	TargetsToRelease.Reserve(AssignedTargets.Num());

	for (auto& AssignedTarget : AssignedTargets)
	{
		if ((AssignedTarget.Value.UsageTags & KeepTags) != 0x00)
		{
			continue;
		}
		else if (!AssignedTarget.Value.Assignee.IsValid() || AssignedTarget.Value.Assignee == Owner)
		{
			TargetsToRelease.Add(AssignedTarget.Key);
		}
	}

	for (UTextureRenderTarget2D* Target : TargetsToRelease)
	{
		ReleaseTarget(Target);
	}
}

//------------------------------------------------------------------------------
void FCompElementRenderTargetPool::ReleaseTaggedTargets(const int32 TargetTags, UObject* Owner)
{
	TArray<UTextureRenderTarget2D*> TargetsToRelease;
	TargetsToRelease.Reserve(AssignedTargets.Num());

	for (auto TargetIt = AssignedTargets.CreateIterator(); TargetIt; ++TargetIt)
	{
		if ((!Owner || Owner == TargetIt.Value().Assignee) && 
			(TargetIt.Value().UsageTags & TargetTags) != 0x00)
		{
			TargetsToRelease.Add(TargetIt.Key());
		}
	}

	for (UTextureRenderTarget2D* Target : TargetsToRelease)
	{
		ReleaseTarget(Target);
	}
}

//------------------------------------------------------------------------------
int32 FCompElementRenderTargetPool::FindAssignedUsageTags(UTextureRenderTarget2D* Target)
{
	int32 AssignedUsageTags = 0x00;
	if (FTargetAssignee* Asignee = AssignedTargets.Find(Target))
	{
		AssignedUsageTags = Asignee->UsageTags;
	}
	return AssignedUsageTags;
}

//------------------------------------------------------------------------------
int32 FCompElementRenderTargetPool::GetTargetCount() const
{
	return AssignedTargets.Num() + RenderTargetPool.Num();
}

//------------------------------------------------------------------------------
void FCompElementRenderTargetPool::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(AssignedTargets);

	for (auto& PooledTarget : RenderTargetPool)
	{
		Collector.AddReferencedObject(PooledTarget.Value.TextureTarget);
	}
}

#if WITH_EDITOR
//------------------------------------------------------------------------------
void FCompElementRenderTargetPool::Tick(float /*DeltaSeconds*/)
{
	ReleaseAssignedTargets(/*Owner =*/nullptr);

	// since we can run in the editor, and could continuously alter the target's render size,
	// we want to flush unused targets (out of fear that they'd never be used again) - targets 
	// regularly used should still be claimed at this point
	if (CVarAutoFlushUnusedTargets.GetValueOnGameThread() > 0)
	{
		FlushStaleTargets();
	}

	for (auto TargetIt = RenderTargetPool.CreateIterator(); TargetIt; ++TargetIt)
	{
		++TargetIt->Value.StaleFrameCount;
	}

	ensure(CVarRenderTargetLimit.GetValueOnGameThread() <= 0 || CVarRenderTargetLimit.GetValueOnGameThread() >= GetTargetCount());
}

//------------------------------------------------------------------------------
bool FCompElementRenderTargetPool::IsTickable() const
{
	return RenderTargetPool.Num() > 0;
}

//------------------------------------------------------------------------------
TStatId FCompElementRenderTargetPool::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCompElementRenderTargetPool, STATGROUP_Tickables);
}
#endif

//------------------------------------------------------------------------------
void FCompElementRenderTargetPool::FlushUnusedTargetPool()
{
	ReleaseAssignedTargets(/*Owner =*/nullptr);

	for (auto& PooledTarget : RenderTargetPool)
	{
		ensureAlways(CVarBreakOnTargetFlush.GetValueOnGameThread() == 0);
		UKismetRenderingLibrary::ReleaseRenderTarget2D(PooledTarget.Value);
	}
	RenderTargetPool.Empty();
}

//------------------------------------------------------------------------------
void FCompElementRenderTargetPool::ForceRePoolAllTargets()
{
	for (auto& Target : AssignedTargets)
	{
		FRenderTargetDesc TargetDesc = { FIntPoint(Target.Key->SizeX, Target.Key->SizeY), Target.Key->RenderTargetFormat };
		RenderTargetPool.Add(TargetDesc, Target.Key.Get());
	}
	AssignedTargets.Empty();
}

//------------------------------------------------------------------------------
void FCompElementRenderTargetPool::FlushAllTargets()
{
	ForceRePoolAllTargets();
	FlushUnusedTargetPool();
}

//------------------------------------------------------------------------------
#if WITH_EDITOR
void FCompElementRenderTargetPool::FlushStaleTargets()
{
	for (auto TargetIt = RenderTargetPool.CreateIterator(); TargetIt; ++TargetIt)
	{
		if (TargetIt->Value.StaleFrameCount >= CVarAutoFlushUnusedTargets.GetValueOnGameThread())
		{
			ensureAlways(CVarBreakOnTargetFlush.GetValueOnGameThread() == 0);
			UKismetRenderingLibrary::ReleaseRenderTarget2D(TargetIt->Value);
			TargetIt.RemoveCurrent();
		}
	}
}
#endif

/* FInheritedTargetPool
  *****************************************************************************/

#include "CompositingElements/InheritedCompositingTargetPool.h"

//------------------------------------------------------------------------------
FInheritedTargetPool::FInheritedTargetPool(UObject* InOwner, FIntPoint InNativeResolution, ETextureRenderTargetFormat InNativeFormat, const FSharedTargetPoolPtr& InInheritedPool, int32 InUsageTags)
	: InheritedPool(InInheritedPool)
	, Owner(InOwner)
	, UsageTags(InUsageTags)
	, NativeTargetResolution(InNativeResolution)
	, NativeTargetFormat(InNativeFormat)
{}

//------------------------------------------------------------------------------
FInheritedTargetPool::FInheritedTargetPool(FInheritedTargetPool& Other, FIntPoint NewTargetResolution, ETextureRenderTargetFormat NewTargetFormat)
	: InheritedPool(Other.InheritedPool)
	, Owner(Other.Owner)
	, UsageTags(Other.UsageTags)
	, NativeTargetResolution(NewTargetResolution)
	, NativeTargetFormat(NewTargetFormat)
{}

//------------------------------------------------------------------------------
bool FInheritedTargetPool::IsValid() const
{
	return Owner.IsValid() && InheritedPool.IsValid();
}

//------------------------------------------------------------------------------
void FInheritedTargetPool::Reset()
{
	Owner.Reset();
	InheritedPool.Reset();
}

//------------------------------------------------------------------------------
UTextureRenderTarget2D* FInheritedTargetPool::RequestRenderTarget(float RenderScale)
{
	FVector2D ScaledDimensions = FVector2D(NativeTargetResolution) * RenderScale;
	return RequestRenderTarget(FIntPoint(ScaledDimensions.X, ScaledDimensions.Y), NativeTargetFormat);
}

//------------------------------------------------------------------------------
UTextureRenderTarget2D* FInheritedTargetPool::RequestRenderTarget(FIntPoint Dimensions, ETextureRenderTargetFormat Format)
{
	if (IsValid())
	{
		return InheritedPool.Pin()->AssignTarget(Owner.Get(), Dimensions, Format, UsageTags);
	}
	return nullptr;
}

//------------------------------------------------------------------------------
bool FInheritedTargetPool::ReleaseRenderTarget(UTextureRenderTarget2D* UsedTarget)
{
	if (InheritedPool.IsValid())
	{
		return InheritedPool.Pin()->ReleaseTarget(UsedTarget);
	}
	return false;
}

//------------------------------------------------------------------------------
FScopedTargetPoolTagAddendum::FScopedTargetPoolTagAddendum(int32 NewTags, FInheritedTargetPool& InTargetPool)
	: TargetPool(InTargetPool)
	, TagsToRestore(InTargetPool.UsageTags)
{
	InTargetPool.UsageTags |= NewTags;
}

//------------------------------------------------------------------------------
FScopedTargetPoolTagAddendum::~FScopedTargetPoolTagAddendum()
{
	TargetPool.UsageTags = TagsToRestore;
}
