// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_EDITOR
#include "TickableEditorObject.h"
#endif

class UTextureRenderTarget2D;
class UObject;
enum ETextureRenderTargetFormat : int;

/* FCompElementRenderTargetPool
 *****************************************************************************/

typedef TSharedPtr<class FCompElementRenderTargetPool> FSharedTargetPoolPtr;
typedef TSharedRef<class FCompElementRenderTargetPool> FSharedTargetPoolRef;
typedef TWeakPtr<  class FCompElementRenderTargetPool> FWeakTargetPoolPtr;

class FCompElementRenderTargetPool : public FGCObject
#if WITH_EDITOR
	, public FTickableEditorObject
#endif
{
public:
	static FSharedTargetPoolRef GetSharedInstance();
	static int32 ExtensionPriority;

	 FCompElementRenderTargetPool(UObject* Outer);
	~FCompElementRenderTargetPool();

public:
	/** Finds a matching render target from the pool, allocates a new one if one doesn't exist. Persists for the Owner object, until released. */
	UTextureRenderTarget2D* AssignTarget(UObject* Owner, FIntPoint Dimensions, ETextureRenderTargetFormat Format, const int32 UsageTags = 0x00);

	/** Returns a specified target to the pool. Assumes the assigned Owner is the one releasing (and taking care of any dangling refs). */
	bool ReleaseTarget(UTextureRenderTarget2D* Target);

	/** Returns all targets assigned to the specified Owner, back to the pool. Items in the optional RetainList will be kept. */
	void ReleaseAssignedTargets(UObject* Owner, const int32 KeepTags = 0x00);
	void ReleaseTaggedTargets(const int32 TargetTags, UObject* Owner = nullptr);

	/** */
	int32 FindAssignedUsageTags(UTextureRenderTarget2D* Target);

	/** */
	int32 GetTargetCount() const;

public:
	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCompElementRenderTargetPool");
	}
	//~ End FGCObject interface

	//~ Begin FTickableEditorObject interface
#if WITH_EDITOR
	virtual void Tick(float DeltaSeconds) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;
#endif
	//~ End FTickableEditorObject interface

private:
	void FlushUnusedTargetPool();
	void ForceRePoolAllTargets();
	void FlushAllTargets();

#if WITH_EDITOR
	void FlushStaleTargets();
#endif

private:
	static FWeakTargetPoolPtr SharedInstance;

	TWeakObjectPtr<UObject> PoolOwner;

	struct FRenderTargetDesc
	{
		FIntPoint Dimensions;
		ETextureRenderTargetFormat Format;

		bool operator==(const FRenderTargetDesc& Rhs) const { return Dimensions == Rhs.Dimensions && Format == Rhs.Format; }
		friend uint32 GetTypeHash(const FRenderTargetDesc& InTargetDesc) { return HashCombine(GetTypeHash(InTargetDesc.Dimensions), GetTypeHash(InTargetDesc.Format)); }
	};
	struct FPooledTarget
	{
		FPooledTarget(UTextureRenderTarget2D* InTextureTarget)
			: TextureTarget(InTextureTarget)
		{}

		operator UTextureRenderTarget2D*() { return TextureTarget; }
		bool operator==(UTextureRenderTarget2D* Rhs) const { return TextureTarget == Rhs; }
		friend bool operator==(UTextureRenderTarget2D* Lhs, const FPooledTarget& Rhs) { return Lhs == Rhs.TextureTarget; }
		
#if WITH_EDITOR
		int32 StaleFrameCount = 0;
#endif 
		TObjectPtr<UTextureRenderTarget2D> TextureTarget = nullptr;
	};
	TMultiMap<FRenderTargetDesc, FPooledTarget>  RenderTargetPool;

	struct FTargetAssignee
	{
		FTargetAssignee(UObject* AssigneeObj, int32 InUsageTags)
			: Assignee(AssigneeObj), UsageTags(InUsageTags) {}

		TWeakObjectPtr<UObject> Assignee;
		int32 UsageTags = 0x00;
	};
	TMap<TObjectPtr<UTextureRenderTarget2D>, FTargetAssignee> AssignedTargets;
};
