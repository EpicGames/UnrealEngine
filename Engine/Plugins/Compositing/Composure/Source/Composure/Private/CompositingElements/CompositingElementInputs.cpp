// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElements/CompositingElementInputs.h"

/* UCompositingMediaInput
 *****************************************************************************/


#include "Materials/MaterialInstanceDynamic.h"
#include "CompositingElements/CompositingElementPassUtils.h"
#include "ComposureConfigSettings.h" // for StaticVideoPlateDebugImage
#include "ComposureInternals.h" // for COMPOSURE_GET_MATERIAL()
#include "UObject/ConstructorHelpers.h"

namespace CompositingMediaInput_Impl
{
	static const FName MediaInputKeyName(TEXT("MediaTransformInputName"));
	static const FName DefaultInputParamName(TEXT("VideoPlate"));
}

UCompositingMediaInput::UCompositingMediaInput()
{
	COMPOSURE_GET_MATERIAL(MaterialInterface, DefaultMaterial, "Media/", "M_VideoPlateDefault");
	COMPOSURE_GET_MATERIAL(MaterialInterface, DefaultTestPlateMaterial, "Media/", "M_StaticVideoPlateDebug");

	MediaTransformMaterial.Material = DefaultMaterial;

	FNamedCompMaterialParam& AddedParamMapping = MediaTransformMaterial.RequiredMaterialParams.Add(CompositingMediaInput_Impl::MediaInputKeyName, CompositingMediaInput_Impl::DefaultInputParamName);
#if WITH_EDITOR
	AddedParamMapping.ParamType = EParamType::MediaTextureParam;
#endif
}

void UCompositingMediaInput::PostInitProperties()
{
	Super::PostInitProperties();

// 	if (ColorConverter == nullptr && !HasAnyFlags(RF_ClassDefaultObject))
// 	{
// 		UCompositingElementMaterialPass* ColorConversionPass = FCompositingElementPassUtils::NewInstancedSubObj<UCompositingElementMaterialPass>(/*Outer =*/this);
// 		ColorConversionPass->Material.Material = DefaultMaterial;
// 
// 		FNamedCompMaterialParam& AddedParamMapping = ColorConversionPass->Material.RequiredMaterialParams.Add(CompositingMediaInput_Impl::MediaInputKeyName, TEXT("VideoPlate"));
// #if WITH_EDITOR
// 		AddedParamMapping.ParamType = EParamType::MediaTextureParam;
// #endif
// 		ColorConverter = ColorConversionPass;
// 	}
}

UTexture* UCompositingMediaInput::GenerateInput_Implementation()
{
	UTexture* MediaTexture = GetMediaTexture();
	UTexture* Result = MediaTexture;
	UMaterialInterface* FallbackMaterial = DefaultMaterial;

	const bool bUseDebugImage = (!MediaTexture && MediaTransformMaterial.Material == DefaultMaterial);
	if (bUseDebugImage)
	{
		const UComposureGameSettings* ConfigSettings = GetDefault<UComposureGameSettings>();
		Result = Cast<UTexture>(ConfigSettings->StaticVideoPlateDebugImage.TryLoad());

		FallbackMaterial = DefaultTestPlateMaterial;
	}

	const bool bUseFallbackMat = bUseDebugImage || MediaTransformMaterial.Material == nullptr;
	if (bUseFallbackMat)
	{
		if (!FallbackMID || FallbackMID->Parent != FallbackMaterial)
		{
			FallbackMID = UMaterialInstanceDynamic::Create(FallbackMaterial, this);
		}
		FallbackMID->SetTextureParameterValue(CompositingMediaInput_Impl::DefaultInputParamName, Result);
		
		// extraneous render pass, but needed since chroma picking cannot sample from a non render target
		UTextureRenderTarget2D* TransformTarget = RequestNativelyFormattedTarget();
		FCompositingElementPassUtils::RenderMaterialToRenderTarget(/*WorldContext =*/this, FallbackMID, TransformTarget);
		Result = TransformTarget;
	}
	else if (MediaTransformMaterial.ApplyParamOverrides(/*TextureLookupTable =*/nullptr))
	{
		const FName InputParamName = MediaTransformMaterial.RequiredMaterialParams.FindChecked(CompositingMediaInput_Impl::MediaInputKeyName);
		MediaTransformMaterial.SetMaterialParam(InputParamName, Result);

		UTextureRenderTarget2D* TransformTarget = RequestNativelyFormattedTarget();
		MediaTransformMaterial.RenderToRenderTarget(/*WorldContext =*/this, TransformTarget);

		Result = TransformTarget;
	}

// 	UCompositingElementMaterialPass* MaterialColorPass = Cast<UCompositingElementMaterialPass>(ColorConverter);
// 	const bool bUseDebugImage = !MediaTexture && (!ColorConverter || (MaterialColorPass && MaterialColorPass->Material.Material == DefaultMaterial));
// 
// 	if (bUseDebugImage)
// 	{
// 		const UComposureGameSettings* ConfigSettings = GetDefault<UComposureGameSettings>();
// 		Result = Cast<UTexture>(ConfigSettings->StaticVideoPlateDebugImage.TryLoad());
// 	}
// 	else if (ColorConverter)
// 	{
// 		if (MaterialColorPass)
// 		{
// 			const FName InputParamName = MaterialColorPass->Material.RequiredMaterialParams.FindChecked(CompositingMediaInput_Impl::MediaInputKeyName);
// 			if (!InputParamName.IsNone())
// 			{
// 				MaterialColorPass->Material.SetMaterialParam(InputParamName, MediaTexture);
// 			}
// 		}
// 
// 		// @TODO - should outputs have access to the PrePassLookupTable? what about PostProcessProxy?
// 		Result = ColorConverter->ApplyTransform(MediaTexture, /*PrePassLookupTable =*/nullptr, /*PostProcessProxy =*/nullptr, SharedTargetPool);
// 	}

	return Result;
}

/* UMediaBundleCompositingInput
 *****************************************************************************/

// #include "MediaBundle.h"
// #include "MediaTexture.h"
// 
// UTexture* UMediaBundleCompositingInput::GetMediaTexture() const
// {
// 	if (MediaSource != nullptr)
// 	{
// 		return MediaSource->GetMediaTexture();
// 	}
// 	return nullptr;
// }

/* UMediaTextureCompositingInput
 *****************************************************************************/

#include "MediaTexture.h"

UMediaTextureCompositingInput::UMediaTextureCompositingInput()
{
	// set to false so the Chroma Key picker works
	bIntermediate = false;
}

UTexture* UMediaTextureCompositingInput::GetMediaTexture() const
{
	return MediaSource;
}

/* UCompositingInputInterfaceProxy
 *****************************************************************************/

#include "ComposureInternals.h" // for 'Composure' log category

UCompositingInputInterface::UCompositingInputInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

void UCompositingInputInterfaceProxy::OnFrameBegin_Implementation(bool bCameraCutThisFrame)
{
	UObject* BoundObject = CompositingInput.GetObject();
	if (BoundObject && BoundObject->GetClass()->ImplementsInterface(UCompositingInputInterface::StaticClass()))
	{
		ICompositingInputInterface::Execute_OnFrameBegin(BoundObject, this, bCameraCutThisFrame);
	}
}

UTexture* UCompositingInputInterfaceProxy::GenerateInput_Implementation()
{
	UObject* BoundObject = CompositingInput.GetObject();
	if (BoundObject && BoundObject->GetClass()->ImplementsInterface(UCompositingInputInterface::StaticClass()))
	{
		UTexture* RenderResult = ICompositingInputInterface::Execute_GenerateInput(BoundObject, this);
		return RenderResult;
	}
	else
	{
		UE_LOGF(Composure, Warning, "Missing composure proxy interface object - inoperable input.");
	}
	return nullptr;
}

void UCompositingInputInterfaceProxy::OnFrameEnd_Implementation()
{
	UObject* BoundObject = CompositingInput.GetObject();
	if (BoundObject && BoundObject->GetClass()->ImplementsInterface(UCompositingInputInterface::StaticClass()))
	{
		ICompositingInputInterface::Execute_OnFrameEnd(BoundObject, this);
	}
}
