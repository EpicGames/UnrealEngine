// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElements/CompositingElementOutputs.h"

/* UColorConverterOutputPass
 *****************************************************************************/

#include "CompositingElements/CompositingElementPassUtils.h" // for NewInstancedSubObj()

void UColorConverterOutputPass::PostInitProperties()
{
	Super::PostInitProperties();

	if (ColorConverter == nullptr && !HasAnyFlags(RF_ClassDefaultObject) && DefaultConverterClass)
	{
		ColorConverter = FCompositingElementPassUtils::NewInstancedSubObj<UCompositingElementTransform>(/*Outer =*/this, DefaultConverterClass);
	}
}

void UColorConverterOutputPass::OnFrameBegin_Implementation(bool bCameraCutThisFrame)
{
	Super::OnFrameBegin_Implementation(bCameraCutThisFrame);
	InternalReset();
}

void UColorConverterOutputPass::Reset_Implementation()
{
	Super::Reset_Implementation();
	InternalReset();
}

void UColorConverterOutputPass::OnDisabled_Implementation()
{
	Super::OnDisabled_Implementation();
	InternalReset();
}

UTexture* UColorConverterOutputPass::ApplyColorTransform(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, FInheritedTargetPool& TargetPool)
{
	UTexture* Result = Input;
	if (ColorConverter && ColorConverter->IsPassEnabled())
	{
		// @TODO - should outputs have access to the PrePassLookupTable? TargetCamera?
		Result = ColorConverter->ApplyTransform(Input, /*PrePassLookupTable =*/nullptr, PostProcessProxy, /*TargetCamera =*/nullptr, TargetPool);
	}

#if WITH_EDITOR
	PreviewResult = Result;
#endif
	return Result;
}

UTexture* UColorConverterOutputPass::ApplyColorTransform(UTexture* RenderResult, UComposurePostProcessingPassProxy* PostProcessProxy, FIntPoint TargetResolution, ETextureRenderTargetFormat TargetFormat)
{
	FInheritedTargetPool OverriddenTargetPool(SharedTargetPool, TargetResolution, TargetFormat);
	return ApplyColorTransform(RenderResult, PostProcessProxy, OverriddenTargetPool);
}

UTexture* UColorConverterOutputPass::ApplyColorTransform(UTexture* RenderResult, UComposurePostProcessingPassProxy* PostProcessProxy)
{
	return ApplyColorTransform(RenderResult, PostProcessProxy, SharedTargetPool);
}

void UColorConverterOutputPass::InternalReset()
{
#if WITH_EDITOR
	PreviewResult = nullptr;
#endif
}

/* UCompositingMediaCaptureOutput
 *****************************************************************************/

#include "CompositingElements/CompositingElementPassUtils.h" // for NewInstancedSubObj(), GetTargetFormatFromPixelFormat(), CopyToTarget()
#include "ComposureInternals.h"
#include "MediaCapture.h"
#include "CompositingElements/CompositingElementTransforms.h" // for UCompositingTonemapPass
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Editor.h"
#endif //WITH_EDITOR

namespace CompositingMediaCaptureOutputUtil
{

#if WITH_EDITOR
	/**
	 * Class made to handle PIE events and cache active MediaCaptures to put it back how it was when PIE ends
	 */
	class FMediaCapturePIEEventHandler
	{
	public:

		static FMediaCapturePIEEventHandler* Get()
		{
			static FMediaCapturePIEEventHandler Instance;
			return &Instance;
		}

		/** Disallow Copying / Moving */
		UE_NONCOPYABLE(FMediaCapturePIEEventHandler);

		~FMediaCapturePIEEventHandler()
		{
			FEditorDelegates::PreBeginPIE.RemoveAll(this);
			FEditorDelegates::EndPIE.RemoveAll(this);
		}

	private:

		FMediaCapturePIEEventHandler()
		{
			FEditorDelegates::PreBeginPIE.AddRaw(this, &FMediaCapturePIEEventHandler::OnPreBeginPIE);
			FEditorDelegates::EndPIE.AddRaw(this, &FMediaCapturePIEEventHandler::OnPIEEnded);
		}

		void OnPreBeginPIE(const bool bIsSimulatingInEditor)
		{
			for (TObjectIterator<UCompositingMediaCaptureOutput> MediaOutIt; MediaOutIt; ++MediaOutIt)
			{
				UCompositingMediaCaptureOutput* CompositingMediaCapture = *MediaOutIt;
				if (CompositingMediaCapture)
				{
					// Cache instances that are currently capturing
					if (CompositingMediaCapture->IsCapturing())
					{
						CompositingOutputToEnable.Add(CompositingMediaCapture);
					}
				}
			}
		}

		void OnPIEEnded(const bool bIsSimulatingInEditor)
		{
			for (const TWeakObjectPtr<UCompositingMediaCaptureOutput>& CachedOutput : CompositingOutputToEnable)
			{
				// Make sure all instances that were enabled prior to PIE are still turned on
				if (UCompositingMediaCaptureOutput* Output = CachedOutput.Get())
				{
					Output->SetPassEnabled(true);
				}
			}
			CompositingOutputToEnable.Empty();
		}

	private:

		TArray<TWeakObjectPtr<UCompositingMediaCaptureOutput>> CompositingOutputToEnable;
	};

#endif //WITH_EDITOR
}

UCompositingMediaCaptureOutput::UCompositingMediaCaptureOutput()
{
	DefaultConverterClass = UCompositingTonemapPass::StaticClass();

#if WITH_EDITOR
	if (IsTemplate() == false)
	{
		//Instanciate util to track PIE media capture state
		CompositingMediaCaptureOutputUtil::FMediaCapturePIEEventHandler::Get();
	}
#endif //WITH_EDITOR
}

bool UCompositingMediaCaptureOutput::IsCapturing() const
{
	return bEnabled && ActiveCapture != nullptr;
}

#if WITH_EDITOR
void UCompositingMediaCaptureOutput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCompositingMediaCaptureOutput, CaptureOutput))
	{
		StopCapture();
	}
}
#endif

void UCompositingMediaCaptureOutput::Reset_Implementation()
{
	StopCapture();
	Super::Reset_Implementation();
}

void UCompositingMediaCaptureOutput::RelayOutput_Implementation(UTexture* RenderResult, UComposurePostProcessingPassProxy* PostProcessProxy)
{
	bool bAppliedColorConversion = false;

	if (RenderResult == nullptr || CaptureOutput == nullptr)
	{
		StopCapture();
	}
	else if (CaptureOutput)
	{
		ETextureRenderTargetFormat OutputFormat = ETextureRenderTargetFormat::RTF_RGB10A2;
		if (FCompositingElementPassUtils::GetTargetFormatFromPixelFormat(CaptureOutput->GetRequestedPixelFormat(), OutputFormat))
		{
			FIntPoint TargetSize = CaptureOutput->GetRequestedSize();
			if ((uint32)TargetSize.GetMax() <= GetMax2DTextureDimension() && TargetSize.GetMin() > 0)
			{
				UTextureRenderTarget2D* OutputTarget = nullptr;
				if (ColorConverter != nullptr)
				{
					UTexture* ColorConversionResult = ApplyColorTransform(RenderResult, PostProcessProxy, TargetSize, OutputFormat);
					bAppliedColorConversion = true;

					OutputTarget = Cast<UTextureRenderTarget2D>(ColorConversionResult);
					if (OutputTarget == nullptr)
					{
						OutputTarget = RequestRenderTarget(TargetSize, OutputFormat);
						FCompositingElementPassUtils::CopyToTarget(/*WorldContext =*/this, RenderResult, OutputTarget);
					}
				}
				else
				{
					OutputTarget = RequestRenderTarget(TargetSize, OutputFormat);
					FCompositingElementPassUtils::CopyToTarget(/*WorldContext =*/this, RenderResult, OutputTarget);
				}

				if (StartCapture(OutputTarget) && ensure(ActiveCapture))
				{
					EMediaCaptureState CaptureState = ActiveCapture->GetState();
					if (CaptureState == EMediaCaptureState::Capturing || CaptureState == EMediaCaptureState::Preparing)
					{
						ActiveCapture->UpdateTextureRenderTarget2D(OutputTarget);
					}
				}
			}
			else
			{
				UE_LOGF(Composure, Warning, "Desired media capture size is too big - unable to convert the render result. Invalid Media Output?");
			}
		}
		else
		{
			UE_LOGF(Composure, Warning, "Requested media capture format is unsupported - unable to convert the render result. Invalid Media Output?");
		}
	}

#if WITH_EDITOR
	if (!bAppliedColorConversion)
	{
		// Run the color conversion for in-editor previewing sake
		PreviewResult = ApplyColorTransform(RenderResult, PostProcessProxy);
	}
#endif 
}

void UCompositingMediaCaptureOutput::OnDisabled_Implementation()
{
	StopCapture();
	Super::OnDisabled_Implementation();
}

void UCompositingMediaCaptureOutput::OnEnabled_Implementation()
{
	Super::OnEnabled_Implementation();
}

bool UCompositingMediaCaptureOutput::StartCapture(UTextureRenderTarget2D* RenderTarget)
{
	if (ActiveCapture == nullptr && RenderTarget)
	{
		if (CaptureOutput)
		{
			for (TObjectIterator<UCompositingMediaCaptureOutput> MediaOutIt; MediaOutIt; ++MediaOutIt)
			{
				UCompositingMediaCaptureOutput* OtherMediaOut = *MediaOutIt;
				if (OtherMediaOut && !OtherMediaOut->IsTemplate() && OtherMediaOut != this)
				{
					if (OtherMediaOut->CaptureOutput == CaptureOutput)
					{
						// Stop the other output
						OtherMediaOut->SetPassEnabled(false);
					}
				}
			}

			ActiveCapture = CaptureOutput->CreateMediaCapture();
			if (ActiveCapture)
			{
				ActiveCapture->CaptureTextureRenderTarget2D(RenderTarget, FMediaCaptureOptions());
			}
		}
	}
	return (ActiveCapture != nullptr);
}

void UCompositingMediaCaptureOutput::StopCapture()
{
	if (ActiveCapture != nullptr)
	{
		ActiveCapture->StopCapture(/*bAllowPendingFrameToBeProcess =*/false);
	}
	ActiveCapture = nullptr;
}

/* URenderTargetCompositingOutput
 *****************************************************************************/

#include "CompositingElements/CompositingElementPassUtils.h" // for CopyToTarget()

void URenderTargetCompositingOutput::RelayOutput_Implementation(UTexture* FinalResult, UComposurePostProcessingPassProxy* /*PostProcessProxy*/)
{
	if (FinalResult && RenderTarget)
	{
		FCompositingElementPassUtils::CopyToTarget(/*WorldContext =*/this, FinalResult, RenderTarget);
	}
}

/* URenderTargetCompositingOutput
 *****************************************************************************/

#include "ImageWriteBlueprintLibrary.h"

UEXRFileCompositingOutput::UEXRFileCompositingOutput()
	: FilenameFormat(TEXT("{frame}"))
	, OutputFrameRate(24, 1)
{}

#if WITH_EDITOR
void UEXRFileCompositingOutput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UEXRFileCompositingOutput, OutputFrameRate))
	{
		//ElapsedCaptureSeconds = 0.0;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UEXRFileCompositingOutput::Reset_Implementation()
{
	InternalReset();
	Super::Reset_Implementation();
}

void UEXRFileCompositingOutput::OnDisabled_Implementation()
{
	InternalReset();
	Super::OnDisabled_Implementation();
}

void UEXRFileCompositingOutput::RelayOutput_Implementation(UTexture* FinalResult, UComposurePostProcessingPassProxy* /*PostProcessProxy*/)
{
	if (FinalResult && !OutputDirectiory.Path.IsEmpty() && !FilenameFormat.IsEmpty())
	{
		bool bShouldCapture = false;
		if (FrameNumber == 0)
		{
			bShouldCapture = true;
		}
		else
		{
			SecondsSinceLastCapture += FApp::GetDeltaTime();
			bShouldCapture = SecondsSinceLastCapture >= OutputFrameRate.AsInterval();
		}

		if (bShouldCapture)
		{
			FImageWriteOptions WriteOptions;
			WriteOptions.Format = EDesiredImageFormat::EXR;
			WriteOptions.bOverwriteFile = true;
			WriteOptions.bAsync = true;
			WriteOptions.CompressionQuality = (int32)Compression;

			const FString FrameIdString = FString::Printf(TEXT("%04d"), FrameNumber);
			FString Filename = FilenameFormat.Replace(TEXT("{frame}"), *FrameIdString);

			UImageWriteBlueprintLibrary::ExportToDisk(FinalResult, OutputDirectiory.Path / Filename, WriteOptions);

			++FrameNumber;
			SecondsSinceLastCapture = 0.0;
		}		
	}
	else
	{
		InternalReset();
	}
}

void UEXRFileCompositingOutput::InternalReset()
{
	FrameNumber = 0;
}
