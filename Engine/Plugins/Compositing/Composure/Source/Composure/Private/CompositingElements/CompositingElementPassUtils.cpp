// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElements/CompositingElementPassUtils.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "UObject/GCObject.h"
#include "UObject/Package.h"
#include "Materials/MaterialInstanceDynamic.h"

struct FCompositingElementAssets : public FGCObject
{
	FSoftObjectPath CopyMatPath;

	static FCompositingElementAssets& Get()
	{
		static FCompositingElementAssets Singleton;
		return Singleton;
	}

	static UMaterialInstanceDynamic* GetCopyMID()
	{
		FCompositingElementAssets& Inst = Get();
		if (Inst.CopyMID == nullptr)
		{
			if (UMaterialInterface* BaseMat = Cast<UMaterialInterface>(Inst.CopyMatPath.TryLoad()))
			{
				Inst.CopyMID = UMaterialInstanceDynamic::Create(BaseMat, GetTransientPackage());
			}
		}
		return Inst.CopyMID;
	}

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		if (CopyMID)
		{
			Collector.AddReferencedObject(CopyMID);
		}
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCompositingElementAssets");
	}

private:
	FCompositingElementAssets()
		: CopyMatPath(TEXT("/Composure/Materials/Output/M_TextureCopy.M_TextureCopy"))
	{}

	TObjectPtr<UMaterialInstanceDynamic> CopyMID = nullptr;
};

template<typename T>
static void FillOutMID_Internal(UMaterialInterface* SrcMaterial, T& TargetMID, UObject* InOuter)
{
	if (SrcMaterial)
	{
		if (!TargetMID || (TargetMID->Parent != SrcMaterial && SrcMaterial && TargetMID))
		{
			if (UMaterialInstanceDynamic* SrcMID = Cast<UMaterialInstanceDynamic>(SrcMaterial))
			{
				TargetMID = SrcMID;
			}
			else
			{
				UObject* MIDOuter = (InOuter == nullptr) ? (TargetMID ? TargetMID->GetOuter() : SrcMaterial->GetOuter()) : InOuter;
				TargetMID = UMaterialInstanceDynamic::Create(SrcMaterial, MIDOuter);
			}
		}
	}
	else
	{
		TargetMID = nullptr;
	}
}

void FCompositingElementPassUtils::FillOutMID(UMaterialInterface* SrcMaterial, UMaterialInstanceDynamic*& TargetMID, UObject* InOuter)
{
	FillOutMID_Internal(SrcMaterial, TargetMID, InOuter);
}

void FCompositingElementPassUtils::FillOutMID(UMaterialInterface* SrcMaterial, TObjectPtr<UMaterialInstanceDynamic>& TargetMID, UObject* InOuter)
{
	FillOutMID_Internal(SrcMaterial, TargetMID, InOuter);
}

void FCompositingElementPassUtils::RenderMaterialToRenderTarget(UObject* WorldContextObj, UMaterialInterface* Material, UTextureRenderTarget2D* RenderTarget)
{
	UKismetRenderingLibrary::DrawMaterialToRenderTarget(WorldContextObj, RenderTarget, Material);
}

bool FCompositingElementPassUtils::CopyToTarget(UObject* WorldContext, UTexture* Src, UTextureRenderTarget2D* Dst)
{
	if (UMaterialInstanceDynamic* CopyMID = FCompositingElementAssets::GetCopyMID())
	{
		CopyMID->SetTextureParameterValue(TEXT("Input"), Src);
		RenderMaterialToRenderTarget(WorldContext, CopyMID, Dst);

		return true;
	}
	return false;
}

bool FCompositingElementPassUtils::GetTargetFormatFromPixelFormat(const EPixelFormat PixelFormat, ETextureRenderTargetFormat& OutRTFormat)
{
	switch (PixelFormat)
	{
		case PF_G8: OutRTFormat = RTF_R8; return true;
		case PF_R8G8: OutRTFormat = RTF_RG8; return true;
		case PF_B8G8R8A8: OutRTFormat = RTF_RGBA8; return true;

		case PF_R16F: OutRTFormat = RTF_R16f; return true;
		case PF_G16R16F: OutRTFormat = RTF_RG16f; return true;
		case PF_FloatRGBA: OutRTFormat = RTF_RGBA16f; return true;

		case PF_R32_FLOAT: OutRTFormat = RTF_R32f; return true;
		case PF_G32R32F: OutRTFormat = RTF_RG32f; return true;
		case PF_A32B32G32R32F: OutRTFormat = RTF_RGBA32f; return true;
		case PF_A2B10G10R10: OutRTFormat = RTF_RGB10A2; return true;
	}
	return false;
}
