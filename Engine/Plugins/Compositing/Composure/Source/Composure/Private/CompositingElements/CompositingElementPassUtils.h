// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "UObject/Object.h"

enum EPixelFormat : uint8;
enum ETextureRenderTargetFormat : int;

class UMaterialInterface;
class UMaterialInstanceDynamic;
class UObject;
class UTextureRenderTarget2D;
class UClass;
class UTexture;

class FCompositingElementPassUtils
{
public:
	/** */
	static void FillOutMID(UMaterialInterface* SrcMaterial, UMaterialInstanceDynamic*& TargetMID, UObject* MIDOuter = nullptr);

	/** */
	static void FillOutMID(UMaterialInterface* SrcMaterial, TObjectPtr<UMaterialInstanceDynamic>& TargetMID, UObject* MIDOuter = nullptr);

	/** */
	static void RenderMaterialToRenderTarget(UObject* WorldContextObj, UMaterialInterface* Material, UTextureRenderTarget2D* RenderTarget);

	/** */
	template<typename T>
	static T* NewInstancedSubObj(UObject* Outer, UClass* Class = nullptr);

	/** */
	static bool CopyToTarget(UObject* WorldContext, UTexture* Src, UTextureRenderTarget2D* Dst);

	/** */
	static bool GetTargetFormatFromPixelFormat(const EPixelFormat PixelFormat, ETextureRenderTargetFormat& OutRTFormat);

private:
};

/* FCompositingElementPassUtils implementations
 *****************************************************************************/

template<typename T>
T* FCompositingElementPassUtils::NewInstancedSubObj(UObject* Outer, UClass* Class)
{
	EObjectFlags PassObjFlags = Outer->GetMaskedFlags(RF_PropagateToSubObjects);
	if (Outer->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		PassObjFlags |= RF_ArchetypeObject;
	}

	if (Class == nullptr)
	{
		Class = T::StaticClass();
	}

	return NewObject<T>(Outer, Class, NAME_None, PassObjFlags, /*Template =*/nullptr);
}
