// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElements/CompositingElementPasses.h"
#include "ComposureInternals.h"
#include "CompositingElement.h" // for ECompPassConstructionType;

/* CompositingElementPasses_Impl
 *****************************************************************************/

namespace CompositingElementPasses_Impl
{
	static bool GetTargetFormatFromPixelFormat(const EPixelFormat PixelFormat, ETextureRenderTargetFormat& OutRTFormat);
}

static bool CompositingElementPasses_Impl::GetTargetFormatFromPixelFormat(const EPixelFormat PixelFormat, ETextureRenderTargetFormat& OutRTFormat)
{
	switch (PixelFormat)
	{
	case PF_G8: OutRTFormat = RTF_R8; return true;
	case PF_R8G8: OutRTFormat = RTF_RG8; return true;
	case PF_B8G8R8A8: OutRTFormat = RTF_RGBA8; return true;

	case PF_R16F: OutRTFormat = RTF_R16f; return true;
	case PF_G16R16F: OutRTFormat = RTF_RG16f; return true;
	case PF_FloatRGBA: OutRTFormat = RTF_RGBA16f; return true;

	case PF_R32_FLOAT: OutRTFormat = RTF_R32f; return true;
	case PF_G32R32F: OutRTFormat = RTF_RG32f; return true;
	case PF_A32B32G32R32F: OutRTFormat = RTF_RGBA32f; return true;
	case PF_A2B10G10R10: OutRTFormat = RTF_RGB10A2; return true;
	}
	return false;
}

/* UCompositingElementPass
 *****************************************************************************/

UCompositingElementPass::UCompositingElementPass()
	: bEnabled(true)
#if WITH_EDITOR
	, ConstructionMethod(ECompPassConstructionType::EditorConstructed)
#endif
{}

void UCompositingElementPass::OnFrameBegin_Implementation(bool /*bCameraCutThisFrame*/)
{
	// for sub-classes to override
}

void UCompositingElementPass::OnFrameEnd_Implementation()
{
	SharedTargetPool.Reset();
}

void UCompositingElementPass::Reset_Implementation()
{
	SharedTargetPool.Reset();
}

void UCompositingElementPass::SetPassEnabled(bool bEnabledIn)
{
	if (bEnabled != bEnabledIn)
	{
		bEnabled = bEnabledIn;
		if (!bEnabled)
		{
			OnDisabled();
		}
		else
		{
			OnEnabled();
		}
	}
}

#if WITH_EDITOR
void UCompositingElementPass::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCompositingElementPass, bEnabled))
	{
		if (!bEnabled)
		{
			OnDisabled();
		}
		else
		{
			OnEnabled();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UCompositingElementPass::SetRenderTargetPool(const FInheritedTargetPool& TargetPool)
{
	SharedTargetPool = TargetPool;
}

void UCompositingElementPass::OnDisabled_Implementation()
{

}

void UCompositingElementPass::OnEnabled_Implementation()
{

}

UTextureRenderTarget2D* UCompositingElementPass::RequestRenderTarget(FIntPoint Dimensions, ETextureRenderTargetFormat Format)
{
	if (ensureMsgf(SharedTargetPool.IsValid(), TEXT("Attempting to allocate a render target without a valid pool - are you doing so outside of OnBegin/EndFrame?")))
	{
		return SharedTargetPool.RequestRenderTarget(Dimensions, Format);
	}
	else
	{
		UE_LOGF(Composure, Warning, "Unable to allocate render target without a target pool. Are you calling this outside of Begin/EndFrame?");
	}
	return nullptr;
}

UTextureRenderTarget2D* UCompositingElementPass::RequestNativelyFormattedTarget(float RenderScale)
{
	if (ensureMsgf(SharedTargetPool.IsValid(), TEXT("Attempting to allocate a render target without a valid pool - are you doing so outside of OnBegin/EndFrame?")))
	{
		return SharedTargetPool.RequestRenderTarget(RenderScale);
	}
	else
	{
		UE_LOGF(Composure, Warning, "Unable to allocate render target without a target pool. Are you calling this outside of Begin/EndFrame?");
	}
	return nullptr;
}

UTextureRenderTarget2D* UCompositingElementPass::RequestRenderTarget(FIntPoint Dimensions, EPixelFormat Format)
{
	ETextureRenderTargetFormat TargetFormat = ETextureRenderTargetFormat::RTF_RGBA16f;
	if (CompositingElementPasses_Impl::GetTargetFormatFromPixelFormat(Format, TargetFormat))
	{
		return RequestRenderTarget(Dimensions, TargetFormat);
	}
	else
	{
		UE_LOGF(Composure, Warning, "Unable to allocate render target - unsupported pixl format: %d", (uint32)Format);
	}
	return nullptr;
}

bool UCompositingElementPass::ReleaseRenderTarget(UTextureRenderTarget2D* AssignedTarget)
{
	return SharedTargetPool.ReleaseRenderTarget(AssignedTarget);
}

/* UCompositingElementInput
 *****************************************************************************/

UTexture* UCompositingElementInput::GenerateInput(const FInheritedTargetPool& InheritedPool)
{
	SetRenderTargetPool(InheritedPool);
	UTexture* Result = GenerateInput();

	// clear to an invalid pool, to catch any attempted allocations outside the scope of this function
	SetRenderTargetPool(FInheritedTargetPool());

	return Result;
}

UTexture* UCompositingElementInput::GenerateInput_Implementation()
{
	// @TODO - Add implementations for all the compositingElementInput types. Currently only the editor default type has the implementation.
	ensureMsgf(false, TEXT("PURE VIRTUAL - Did you forget to override GenerateInput() for this CompositingElementInput?"));
	UE_LOGF(Composure, Error, "GenerateInput() not overriden for %ls.", *GetClass()->GetName());
	return nullptr;
}

/* UCompositingElementTransform
 *****************************************************************************/


UTexture* UCompositingElementTransform::ApplyTransform(UTexture* Input, ICompositingTextureLookupTable* InPrePassLookupTable, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* TargetCamera, const FInheritedTargetPool& InheritedPool)
{
	PrePassLookupTable = InPrePassLookupTable;

	SetRenderTargetPool(InheritedPool);
	UTexture* Result = ApplyTransform(Input, PostProcessProxy, TargetCamera);

	// clear to an invalid pool, to catch any attempted allocations outside the scope of this function
	SetRenderTargetPool(FInheritedTargetPool());

	PrePassLookupTable = nullptr;
	return Result;
}

UTexture* UCompositingElementTransform::ApplyTransform_Implementation(UTexture* /*Input*/, UComposurePostProcessingPassProxy* /*PostProcessProxy*/, ACameraActor* /*TargetCamera*/)
{
	ensureMsgf(false, TEXT("PURE VIRTUAL - Did you forget to override TransformTarget() for this CompositingElementTransform?"));
	UE_LOGF(Composure, Error, "TransformTarget() not overriden for %ls.", *GetClass()->GetName());
	return nullptr;
}

UTexture* UCompositingElementTransform::FindNamedPrePassResult(FName LookupName)
{
	if ( ensureMsgf(PrePassLookupTable, TEXT("Calling FindNamePrePassResult() outside the scope of UCompositingElementTransform::ApplyTransform().")) )
	{
		UTexture* FoundResult = nullptr;
		PrePassLookupTable->FindNamedPassResult(LookupName, FoundResult);

		return FoundResult;
	}
	else
	{
		UE_LOGF(Composure, Warning, "Calling FindNamePrePassResult() outside the scope of UCompositingElementTransform::ApplyTransform() - this will always fail.");
	}
	return nullptr;
}

/* UCompositingElementOutput
 *****************************************************************************/

void UCompositingElementOutput::RelayOutput(UTexture* FinalResult, UComposurePostProcessingPassProxy* PostProcessProxy, const FInheritedTargetPool& InheritedPool)
{
	SetRenderTargetPool(InheritedPool);
	RelayOutput(FinalResult, PostProcessProxy);

	// clear to an invalid pool, to catch any attempted allocations outside the scope of this function
	SetRenderTargetPool(FInheritedTargetPool());
}

void UCompositingElementOutput::RelayOutput_Implementation(UTexture* /*FinalResult*/, UComposurePostProcessingPassProxy* /*PostProcessProxy*/)
{
	ensureMsgf(false, TEXT("PURE VIRTUAL - Did you forget to override RelayOutput() for this CompositingElementOutput?"));
	UE_LOGF(Composure, Error, "RelayOutput() not overriden for %ls.", *GetClass()->GetName());
}
