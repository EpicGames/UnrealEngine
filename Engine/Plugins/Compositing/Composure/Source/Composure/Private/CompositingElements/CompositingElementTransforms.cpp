// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElements/CompositingElementTransforms.h"
#include "ComposurePostProcessingPassProxy.h" // IWYU pragma: keep
#include "Materials/Material.h"

/* TCompositingTargetSwapChain
 *****************************************************************************/

template <typename AllocatorType>
struct TCompositingTargetSwapChain
{
public:
	TCompositingTargetSwapChain(const AllocatorType& InAllocator)
		: Allocator(InAllocator)
	{}

	UTextureRenderTarget2D*& Get()
	{
		return GetTarget(InternalIndex);
	}

	UTextureRenderTarget2D*& GetTarget(uint32 Index)
	{
		Index = Index % 2;

		UTextureRenderTarget2D* Target = Buffers[Index];
		if (Target == nullptr)
		{
			Buffers[Index] = Allocator.AllocateTarget();
		}

		return Buffers[Index];
	}

	operator UTextureRenderTarget2D*()
	{
		return Get();
	}

	void operator++()
	{
		InternalIndex = (InternalIndex + 1) % 2;
	}

	operator bool()
	{
		return Buffers[InternalIndex] != nullptr;
	}

	UTextureRenderTarget2D* Release()
	{
		const int32 LastRenderIndex = (InternalIndex + 1) % 2;

		UTextureRenderTarget2D* FinalTarget = Buffers[LastRenderIndex];
		// do not release this target, return it instead for use - it is the "result"
		Buffers[LastRenderIndex] = nullptr;
		
		if (UTextureRenderTarget2D* IntermediateTarget = Buffers[InternalIndex])
		{
			Allocator.ReleaseTarget(IntermediateTarget);
			Buffers[InternalIndex] = nullptr;
		}
		
		return FinalTarget;
	}

private:
	int32 InternalIndex = 0;
	UTextureRenderTarget2D* Buffers[2] = { nullptr };

	AllocatorType Allocator;
};

struct FScaledTargetAllocator
{
public:
	FScaledTargetAllocator(FInheritedTargetPool& InTargetPool, float InTargetScale)
		: TargetPoolRef(InTargetPool)
		, TargetScale(InTargetScale)
	{}

	UTextureRenderTarget2D* AllocateTarget()
	{
		return TargetPoolRef.RequestRenderTarget(TargetScale);
	}

	void ReleaseTarget(UTextureRenderTarget2D* Target)
	{
		TargetPoolRef.ReleaseRenderTarget(Target);
	}

private:
	FInheritedTargetPool& TargetPoolRef;
	float TargetScale = 1.f;
};

class FCompositingTargetSwapChain : public TCompositingTargetSwapChain<FScaledTargetAllocator>
{
public:
	FCompositingTargetSwapChain(FInheritedTargetPool& InTargetPool, float InTargetScale = 1.f)
		: TCompositingTargetSwapChain<FScaledTargetAllocator>(FScaledTargetAllocator(InTargetPool, InTargetScale))
	{}
};

/* UCompositingPostProcessPass
 *****************************************************************************/

#include "ComposureInternals.h" // for 'Composure' log category

UTexture* UCompositingPostProcessPass::ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* /*TargetCamera*/)
{
	FCompositingTargetSwapChain SwapChain = FCompositingTargetSwapChain(SharedTargetPool, RenderScale);
	RenderPostPassesToSwapChain(Input, PostProcessProxy, SwapChain);

	UTexture* FinalResult = Input;
	if (UTextureRenderTarget2D* PostPassesResult = SwapChain.Release())
	{
		FinalResult = PostPassesResult;
	}
	return FinalResult;
}

void UCompositingPostProcessPass::RenderPostPassesToSwapChain(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, FCompositingTargetSwapChain& TargetSwapChain)
{
	if (PostProcessProxy)
	{
		UTexture* PassInput = Input;
		for (UComposurePostProcessPassPolicy* PostPass : PostProcessPasses)
		{
			if (PostPass)
			{
				PostProcessProxy->SetOutputRenderTarget(TargetSwapChain);
				PostProcessProxy->Execute(PassInput, PostPass);

				PassInput = TargetSwapChain;
				++TargetSwapChain;
			}
		}
		PostProcessProxy->SetOutputRenderTarget(nullptr);
	}
	else if (PostProcessPasses.Num() > 0)
	{
		UE_LOGF(Composure, Error, "Missing PostProcessProxy - unable to apply post-process.")
	}
}

/* UCompositingElementMaterialPass
 *****************************************************************************/

UTexture* UCompositingElementMaterialPass::ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* /*TargetCamera*/)
{
	FCompositingTargetSwapChain SwapChain = FCompositingTargetSwapChain(SharedTargetPool, RenderScale);

	UTexture* Result = Input;
	if (Material.ApplyParamOverrides(PrePassLookupTable))
	{
		ApplyMaterialParams(Material.GetMID());

		Material.RenderToRenderTarget(/*WorldContext =*/this, SwapChain);
		Result = SwapChain.Get();

		++SwapChain;
	}
	
	RenderPostPassesToSwapChain(Result, PostProcessProxy, SwapChain);

	if (UTextureRenderTarget2D* PostPassesResult = SwapChain.Release())
	{
		Result = PostPassesResult;
	}
	return Result;
}

void UCompositingElementMaterialPass::SetMaterialInterface(UMaterialInterface* NewMaterial)
{
	if (NewMaterial)
	{
		Material.Material = Cast<UMaterialInterface>(NewMaterial->GetMaterial());
#if WITH_EDITORONLY_DATA
		Material.UpdateProxyMap();
#endif
	}
}

bool UCompositingElementMaterialPass::SetParameterMapping(FName TextureParamName, FName ComposureLayerName)
{
	bool bIsSetSuccess = false;
	if (Material.ParamPassMappings.Num() > 0 && Material.ParamPassMappings.Contains(TextureParamName))
	{
		Material.ParamPassMappings[TextureParamName] = ComposureLayerName;
		bIsSetSuccess = true;
	}
	return bIsSetSuccess;
}



/* UCompositingTonemapPass
 *****************************************************************************/

#include "CompositingElements/CompositingElementPassUtils.h" // for NewInstancedSubObj()
#include "ComposureTonemapperPass.h"

UTexture* UCompositingTonemapPass::ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* /*TargetCamera*/)
{
	UTexture* Result = Input;
	if (PostProcessProxy)
	{
		if (TonemapPolicy == nullptr)
		{
			TonemapPolicy = FCompositingElementPassUtils::NewInstancedSubObj<UComposureTonemapperPassPolicy>(this);
		}

		TonemapPolicy->ColorGradingSettings = ColorGradingSettings;
		TonemapPolicy->FilmStockSettings = FilmStockSettings;
		TonemapPolicy->ChromaticAberration = ChromaticAberration;

		UTextureRenderTarget2D* TonemapperTarget = RequestNativelyFormattedTarget();
		PostProcessProxy->SetOutputRenderTarget(TonemapperTarget);
		PostProcessProxy->Execute(Input, TonemapPolicy);
		PostProcessProxy->SetOutputRenderTarget(nullptr);

		Result = TonemapperTarget;
	}
	else
	{
		UE_LOGF(Composure, Error, "Missing PostProcessProxy - unable to apply tonemapping.")
	}
	return Result;
}

/* UMultiPassChromaKeyer
 *****************************************************************************/

#include "UObject/ConstructorHelpers.h"

namespace MultiPassChromaKeyer_Impl
{
	static const FName ColorPlateKeyName(TEXT("ColorPlateParamName"));
	static const FName KeyColorKeyName(TEXT("ColorKeyParamName"));
	static const FName KeyedResultInputName(TEXT("KeyedResultInputName"));
}

UMultiPassChromaKeyer::UMultiPassChromaKeyer()
{
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> GKeyerMaterial(TEXT("/Composure/Materials/ChromaKeying/M_SinglePassChromaKeyer"));
	KeyerMaterial.Material = Cast<UMaterialInterface>(GKeyerMaterial.Object);

	KeyerMaterial.RequiredMaterialParams.Add(MultiPassChromaKeyer_Impl::ColorPlateKeyName, TEXT("LinearColorPlate"));
	KeyerMaterial.RequiredMaterialParams.Add(MultiPassChromaKeyer_Impl::KeyColorKeyName, TEXT("KeyColor"));
	KeyerMaterial.RequiredMaterialParams.Add(MultiPassChromaKeyer_Impl::KeyedResultInputName, TEXT("PrevKeyerResult"));

#if WITH_EDITOR
	KeyerMaterial.RequiredMaterialParams[MultiPassChromaKeyer_Impl::ColorPlateKeyName].ParamType = EParamType::TextureParam;
	KeyerMaterial.RequiredMaterialParams[MultiPassChromaKeyer_Impl::KeyColorKeyName].ParamType = EParamType::VectorParam;
	KeyerMaterial.RequiredMaterialParams[MultiPassChromaKeyer_Impl::KeyedResultInputName].ParamType = EParamType::TextureParam;
#endif

	static ConstructorHelpers::FObjectFinder<UTexture> GDefaultResultTexture(TEXT("/Engine/EngineResources/WhiteSquareTexture.WhiteSquareTexture"));
	DefaultWhiteTexture = Cast<UTexture>(GDefaultResultTexture.Object);
}

UTexture* UMultiPassChromaKeyer::ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* /*TargetCamera*/)
{
	FCompositingTargetSwapChain SwapChain = FCompositingTargetSwapChain(SharedTargetPool, /*RenderScale =*/1.f);

	UTexture* Result = Input;
	if (KeyerMaterial.ApplyParamOverrides(PrePassLookupTable))
	{
		const FName InputParamName = KeyerMaterial.RequiredMaterialParams.FindChecked(MultiPassChromaKeyer_Impl::ColorPlateKeyName);
		KeyerMaterial.SetMaterialParam(InputParamName, Input);

		const FName ResultParamName = KeyerMaterial.RequiredMaterialParams.FindChecked(MultiPassChromaKeyer_Impl::KeyedResultInputName);
		KeyerMaterial.SetMaterialParam(ResultParamName, DefaultWhiteTexture);

		for (const FLinearColor& Key : KeyColors)
		{
			const FName ColorParamName = KeyerMaterial.RequiredMaterialParams.FindChecked(MultiPassChromaKeyer_Impl::KeyColorKeyName);
			KeyerMaterial.SetMaterialParam(ColorParamName, Key);
			
			KeyerMaterial.RenderToRenderTarget(/*WorldContext =*/this, SwapChain);
			Result = SwapChain.Get();
			KeyerMaterial.SetMaterialParam(ResultParamName, Result);

			++SwapChain;
		}
	}

	if (UTextureRenderTarget2D* ChromaKeyingResult = SwapChain.Release())
	{
		Result = ChromaKeyingResult;
	}
	return Result;
}


/* UMultiPassDespill
 *****************************************************************************/

#include "UObject/ConstructorHelpers.h"

LLM_DEFINE_TAG(Composure_MultiPassDespill);

namespace MultiPassDespill_Impl
{
	static const FName ColorPlateKeyName(TEXT("ColorPlateParamName"));
	static const FName KeyColorKeyName(TEXT("ColorKeyParamName"));
	static const FName KeyedResultInputName(TEXT("KeyedResultInputName"));
}

UMultiPassDespill::UMultiPassDespill()
{
	LLM_SCOPE_BYTAG(Composure_MultiPassDespill);

	static ConstructorHelpers::FObjectFinder<UMaterialInterface> GKeyerMaterial(TEXT("/Composure/Materials/ChromaKeying/M_SinglePassDespill"));
	KeyerMaterial.Material = Cast<UMaterialInterface>(GKeyerMaterial.Object);

	KeyerMaterial.RequiredMaterialParams.Add(MultiPassDespill_Impl::ColorPlateKeyName, TEXT("LinearColorPlate"));
	KeyerMaterial.RequiredMaterialParams.Add(MultiPassDespill_Impl::KeyColorKeyName, TEXT("KeyColor"));
	KeyerMaterial.RequiredMaterialParams.Add(MultiPassDespill_Impl::KeyedResultInputName, TEXT("PrevKeyerResult"));

#if WITH_EDITOR
	KeyerMaterial.RequiredMaterialParams[MultiPassDespill_Impl::ColorPlateKeyName].ParamType = EParamType::TextureParam;
	KeyerMaterial.RequiredMaterialParams[MultiPassDespill_Impl::KeyColorKeyName].ParamType = EParamType::VectorParam;
	KeyerMaterial.RequiredMaterialParams[MultiPassDespill_Impl::KeyedResultInputName].ParamType = EParamType::TextureParam;
#endif

	static ConstructorHelpers::FObjectFinder<UTexture> GDefaultResultTexture(TEXT("/Engine/Functions/Engine_MaterialFunctions02/PivotPainter2/Black_1x1_EXR_Texture.Black_1x1_EXR_Texture"));
	DefaultWhiteTexture = Cast<UTexture>(GDefaultResultTexture.Object);
}

UTexture* UMultiPassDespill::ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* /*TargetCamera*/)
{
	LLM_SCOPE_BYTAG(Composure_MultiPassDespill);

	FCompositingTargetSwapChain SwapChain = FCompositingTargetSwapChain(SharedTargetPool, /*RenderScale =*/1.f);

	UTexture* Result = Input;
	if (KeyerMaterial.ApplyParamOverrides(PrePassLookupTable))
	{
		const FName InputParamName = KeyerMaterial.RequiredMaterialParams.FindChecked(MultiPassDespill_Impl::ColorPlateKeyName);
		KeyerMaterial.SetMaterialParam(InputParamName, Input);

		const FName ResultParamName = KeyerMaterial.RequiredMaterialParams.FindChecked(MultiPassDespill_Impl::KeyedResultInputName);
		KeyerMaterial.SetMaterialParam(ResultParamName, DefaultWhiteTexture);

		for (const FLinearColor& Key : KeyColors)
		{
			const FName ColorParamName = KeyerMaterial.RequiredMaterialParams.FindChecked(MultiPassDespill_Impl::KeyColorKeyName);
			KeyerMaterial.SetMaterialParam(ColorParamName, Key);
			
			KeyerMaterial.RenderToRenderTarget(/*WorldContext =*/this, SwapChain);
			Result = SwapChain.Get();
			KeyerMaterial.SetMaterialParam(ResultParamName, Result);

			++SwapChain;
		}
	}

	if (UTextureRenderTarget2D* DespillResult = SwapChain.Release())
	{
		Result = DespillResult;
	}
	return Result;
}



/* UAlphaTransformPass
*****************************************************************************/

#include "ComposureInternals.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "CompositingElements/CompositingElementPassUtils.h"

UAlphaTransformPass::UAlphaTransformPass()
{
	COMPOSURE_GET_MATERIAL(MaterialInterface, DefaultMaterial, "Compositing/", "M_AlphaScale");
}

UTexture* UAlphaTransformPass::ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* /*PostProcessProxy*/, ACameraActor* /*TargetCamera*/)
{
	if (!AlphaTransformMID && DefaultMaterial)
	{
		AlphaTransformMID = UMaterialInstanceDynamic::Create(DefaultMaterial, /*Outer =*/this);
	}

	UTexture* Result = Input;
	if (AlphaTransformMID)
	{
		AlphaTransformMID->SetTextureParameterValue(TEXT("Input"), Input);
		AlphaTransformMID->SetScalarParameterValue(TEXT("AlphaScale"), AlphaScale);

		UTextureRenderTarget2D* TransformTarget = RequestNativelyFormattedTarget();
		FCompositingElementPassUtils::RenderMaterialToRenderTarget(/*WorldContext =*/this, AlphaTransformMID, TransformTarget);
		Result = TransformTarget;
	}

	return Result;
}

/* UOpenColorIOColorTransformPass
*****************************************************************************/

#include "OpenColorIORendering.h"

UTexture* UCompositingOpenColorIOPass::ApplyTransform_Implementation(UTexture* Input, UComposurePostProcessingPassProxy* PostProcessProxy, ACameraActor* /*TargetCamera*/)
{
	UTextureRenderTarget2D* OutputTarget = RequestNativelyFormattedTarget();

	FOpenColorIORendering::ApplyColorTransform(GetWorld(), ColorConversionSettings, Input, OutputTarget);

	return OutputTarget;
}
