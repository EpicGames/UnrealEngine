// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElements/CompositingMaterialPass.h"
#include "CompositingElements/ICompositingTextureLookupTable.h"
#include "Materials/MaterialInstanceDynamic.h"

/* FCompositingMaterial
 *****************************************************************************/

#include "CompositingElements/CompositingElementPassUtils.h" // for FillOutMID(), RenderMaterialToRenderTarget()
#include "ComposureConfigSettings.h" // for GetFallbackCompositingTexture()
#include "Engine/Texture.h"

static TAutoConsoleVariable<int32> CVarUseBlackForDisabledPasses(
	TEXT("r.Composure.CompositingElements.UseBlackForDisabledPasses"),
	1,
	TEXT("With this enabled, if a compositing material's source element is disabled, then we use a hardcoded black/transparent ")
	TEXT("texture in its place. If you disable this, then it will use whatever the default sampler texture is in the material."));


FCompositingMaterial::FCompositingMaterial()
	: Material(nullptr)
	, CachedMID(nullptr)
{
}

bool FCompositingMaterial::ApplyParamOverrides(const ICompositingTextureLookupTable* TextureLookupTable)
{
	UMaterialInstanceDynamic* MatInstance = GetMID();
	if (MatInstance)
	{
		// To keep stale texture resources from being set as mat parameters, clear each frame
		// (since we're working in editor, resources could be added/removed dynamically).
		ResetMaterial();

		if (bParamsModified)
		{
			for (auto& ScalarOverride : ScalarParamOverrides)
			{
				MatInstance->SetScalarParameterValue(ScalarOverride.Key, ScalarOverride.Value);
			}

			for (auto& VecOverride : VectorParamOverrides)
			{
				MatInstance->SetVectorParameterValue(VecOverride.Key, VecOverride.Value);
			}

			for (TPair<FName, TObjectPtr<UTexture>>& TextureOverride : TextureParamOverrides)
			{
				MatInstance->SetTextureParameterValue(TextureOverride.Key, TextureOverride.Value.Get());
			}

			bParamsModified = false;
		}

		auto GetLookupName = [this](FName ParamName)->FName
		{
			FName* MappingPtr = ParamPassMappings.Find(ParamName);
			if (MappingPtr && !MappingPtr->IsNone())
			{
				return *MappingPtr;
			}
			return ParamName;
		};
		
		TArray<FMaterialParameterInfo> TexParamInfos;
		TArray<FGuid> TexParamIds;
		MatInstance->GetAllTextureParameterInfo(TexParamInfos, TexParamIds);

		const bool bUseFallbackBlackTexture = CVarUseBlackForDisabledPasses.GetValueOnGameThread() != 0;
		UTexture* FallbackTexture = nullptr;
		if (bUseFallbackBlackTexture)
		{
			FallbackTexture = UComposureGameSettings::GetFallbackCompositingTexture();
		}

		for (const FMaterialParameterInfo& ParamInfo : TexParamInfos)
		{
			UTexture* TextureValue = nullptr;
			const bool bFound = TextureLookupTable ? TextureLookupTable->FindNamedPassResult(GetLookupName(ParamInfo.Name), TextureValue) : false;

			if (bFound)
			{
				if (!TextureValue)
				{
					if (FallbackTexture)
					{
						TextureValue = FallbackTexture;
					}
					else
					{
						MatInstance->GetTextureParameterDefaultValue(ParamInfo, TextureValue);
					}
				}
				MatInstance->SetTextureParameterValue(ParamInfo.Name, TextureValue);
			}
		}
	}
	return (MatInstance != nullptr);
}

void FCompositingMaterial::ResetMaterial()
{
	UMaterialInstanceDynamic* MatInstance = GetMID();
	if (MatInstance)
	{
		MatInstance->ClearParameterValues();
		bParamsModified = true;
	}
}

void FCompositingMaterial::RenderToRenderTarget(UObject* WorldContext, UTextureRenderTarget2D* Target) 
{
	UMaterialInstanceDynamic* MatInstance = GetMID();
	if (MatInstance)
	{
		FCompositingElementPassUtils::RenderMaterialToRenderTarget(WorldContext, MatInstance, Target);
	}
	else if (Material)
	{
		FCompositingElementPassUtils::RenderMaterialToRenderTarget(WorldContext, Material, Target);
	}
}

bool FCompositingMaterial::SetMaterialParam(const FName ParamName, float ScalarValue)
{
	UMaterialInstanceDynamic* MatInstance = GetMID();
	if (MatInstance)
	{
		MatInstance->SetScalarParameterValue(ParamName, ScalarValue);
		return true;
	}
	return false;
}

bool FCompositingMaterial::SetMaterialParam(const FName ParamName, FLinearColor VectorValue)
{
	UMaterialInstanceDynamic* MatInstance = GetMID();
	if (MatInstance)
	{
		MatInstance->SetVectorParameterValue(ParamName, VectorValue);
		return true;
	}
	return false;
}

bool FCompositingMaterial::SetMaterialParam(const FName ParamName, UTexture* TextureValue)
{
	UMaterialInstanceDynamic* MatInstance = GetMID();
	if (MatInstance)
	{
		MatInstance->SetTextureParameterValue(ParamName, TextureValue);
		return true;
	}
	return false;
}

void FCompositingMaterial::SetScalarOverride(const FName ParamName, const float ParamVal)
{
	ScalarParamOverrides.Add(ParamName, ParamVal);
	if (CachedMID)
	{
		CachedMID->SetScalarParameterValue(ParamName, ParamVal);
	}
	else
	{
		bParamsModified = true;
	}
}

bool FCompositingMaterial::GetScalarOverride(const FName ParamName, float& OutParamVal)
{
	if (float* ExistingOverride = ScalarParamOverrides.Find(ParamName))
	{
		OutParamVal = *ExistingOverride;
		return true;
	}
	return false;
}

void FCompositingMaterial::ResetScalarOverride(const FName ParamName)
{
	ScalarParamOverrides.Remove(ParamName);

	float DefaultVal = 0.f;
	if (CachedMID && CachedMID->GetScalarParameterDefaultValue(ParamName, DefaultVal))
	{
		CachedMID->SetScalarParameterValue(ParamName, DefaultVal);
	}
}

void FCompositingMaterial::SetVectorOverride(const FName ParamName, const FLinearColor ParamVal)
{
	VectorParamOverrides.Add(ParamName, ParamVal);
	if (CachedMID)
	{
		CachedMID->SetVectorParameterValue(ParamName, ParamVal);
	}
	else
	{
		bParamsModified = true;
	}
}

bool FCompositingMaterial::GetVectorOverride(const FName ParamName, FLinearColor& OutParamVal)
{
	if (FLinearColor* ExistingOverride = VectorParamOverrides.Find(ParamName))
	{
		OutParamVal = *ExistingOverride;
		return true;
	}
	return false;
}

void FCompositingMaterial::ResetVectorOverride(const FName ParamName)
{
	VectorParamOverrides.Remove(ParamName);

	FLinearColor DefaultVal;
	if (CachedMID && CachedMID->GetVectorParameterDefaultValue(ParamName, DefaultVal))
	{
		CachedMID->SetVectorParameterValue(ParamName, DefaultVal);
	}
}

void FCompositingMaterial::SetTextureOverride(const FName ParamName, UTexture* ParamVal)
{
	TextureParamOverrides.Add(ParamName, ParamVal);
	if (CachedMID)
	{
		CachedMID->SetTextureParameterValue(ParamName, ParamVal);
	}
	else
	{
		bParamsModified = true;
	}
}

bool FCompositingMaterial::GetTextureOverride(const FName ParamName, UTexture*& OutParamVal)
{
	if (TObjectPtr<UTexture>* ExistingOverride = TextureParamOverrides.Find(ParamName))
	{
		OutParamVal = *ExistingOverride;
		return true;
	}
	return false;
}

void FCompositingMaterial::ResetTextureOverride(const FName ParamName)
{
	TextureParamOverrides.Remove(ParamName);

	UTexture* DefaultVal;
	if (CachedMID && CachedMID->GetTextureParameterDefaultValue(ParamName, DefaultVal))
	{
		CachedMID->SetTextureParameterValue(ParamName, DefaultVal);
	}
}

void FCompositingMaterial::ResetAllParamOverrides()
{
	while (ScalarParamOverrides.Num() > 0)
	{
		auto ParamIt = ScalarParamOverrides.CreateIterator();
		ResetScalarOverride(ParamIt.Key());
	}

	while (VectorParamOverrides.Num() > 0)
	{
		auto ParamIt = VectorParamOverrides.CreateIterator();
		ResetVectorOverride(ParamIt.Key());
	}

	bParamsModified = true;
}

UMaterialInstanceDynamic* FCompositingMaterial::GetMID()
{
	UMaterialInstanceDynamic* OldMid = CachedMID;
	FCompositingElementPassUtils::FillOutMID(Material, CachedMID);
	// dirty the params, so they get reset
	bParamsModified |= (OldMid != CachedMID);

	return CachedMID;
}

#if WITH_EDITORONLY_DATA
void FCompositingMaterial::UpdateProxyMap()
{
	VectorOverrideProxies.Empty();

	if (Material)
	{
		TArray<FMaterialParameterInfo> OutVectorParameterInfo;
		TArray<FGuid> VectorGuids;
		Material->GetAllVectorParameterInfo(OutVectorParameterInfo, VectorGuids);

		VectorOverrideProxies.Empty(OutVectorParameterInfo.Num());

		for (FMaterialParameterInfo VectorParam : OutVectorParameterInfo)
		{
			if (EditorHiddenParams.Contains(VectorParam.Name))
			{
				continue;
			}

			FLinearColor OutVal;
			if (!GetVectorOverride(VectorParam.Name, OutVal))
			{
				// Not found in overrides, just get default
				Material->GetVectorParameterDefaultValue(VectorParam, OutVal);
			}
			VectorOverrideProxies.Add(VectorParam.Name, OutVal);
		}

		TArray<FMaterialParameterInfo> OutTextureParameterInfo;
		TArray<FGuid> TextureGuids;
		Material->GetAllTextureParameterInfo(OutTextureParameterInfo, TextureGuids);

		TArray<FName> KeysToRemove;

		//Remove missing entries
		for (TPair<FName, FName> MaterialOverride : ParamPassMappings)
		{
			bool Found = false;
			for (FMaterialParameterInfo info : OutTextureParameterInfo)
			{
				if (info.Name == MaterialOverride.Key) Found = true;
			}
			if (!Found) KeysToRemove.Add(MaterialOverride.Key);
		}
		for (FName Key : KeysToRemove)
		{
			ParamPassMappings.Remove(Key);
		}

		//Add new entries
		for (FMaterialParameterInfo info : OutTextureParameterInfo)
		{
			if (!RequiredMaterialParams.Contains(info.Name))
			{
				ParamPassMappings.FindOrAdd(info.Name);
			}
		}
	}
}
#endif // WITH_EDITORONLY_DATA
