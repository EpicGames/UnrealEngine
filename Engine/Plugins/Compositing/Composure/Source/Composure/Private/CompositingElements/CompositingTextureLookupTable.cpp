// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElements/CompositingTextureLookupTable.h"
#include "HAL/IConsoleManager.h"

static FAutoConsoleVariable CVarUserPrePassParamName(
	TEXT("r.Composure.CompositingElements.InternalPrePassParamName"),
	TEXT("Input"),
	TEXT("For compositing elements, each material pass can ganerally reference the pass that came before it using this predefined parameter name."));

UCompositingTextureLookupTable::UCompositingTextureLookupTable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

/* CompositingTextureLookupTable_Impl
 *****************************************************************************/

namespace CompositingTextureLookupTable_Impl
{
	static const FName PrePassAliases[] =
	{
		TEXT("Self"),
		TEXT("PrePass")
	};

	static bool  IsLookupNameAPrePassAlias(FName LookupName);
	static FName GetAuthoratativePrePassLookupName();
}

static bool CompositingTextureLookupTable_Impl::IsLookupNameAPrePassAlias(FName LookupName)
{
	for (int32 AliasIndex = 0; AliasIndex < UE_ARRAY_COUNT(PrePassAliases); ++AliasIndex)
	{
		if (LookupName == PrePassAliases[AliasIndex])
		{
			return true;
		}
	}
	return LookupName == GetAuthoratativePrePassLookupName();
}

static FName CompositingTextureLookupTable_Impl::GetAuthoratativePrePassLookupName()
{
	return *CVarUserPrePassParamName->GetString();
}

/* FCompositingTextureLookupTable
 *****************************************************************************/

void FCompositingTextureLookupTable::RegisterPassResult(FName KeyName, UTexture* Result, const int32 InUsageTags)
{
	FTaggedTexture* ExistingResult = LookupTable.Find(KeyName);
	if (ExistingResult && (ExistingResult->Texture == Result))
	{
		ExistingResult->UsageTags |= InUsageTags;
	}
	else
	{
		FTaggedTexture TaggedTexture;
		TaggedTexture.UsageTags = InUsageTags;
		TaggedTexture.Texture = Result;

		LookupTable.Add(KeyName, TaggedTexture);
	}
}

void FCompositingTextureLookupTable::SetMostRecentResult(UTexture* Result)
{
	using namespace CompositingTextureLookupTable_Impl;
	RegisterPassResult(GetAuthoratativePrePassLookupName(), Result);
}

void FCompositingTextureLookupTable::ResetAll()
{
	Empty();
	ClearLinkedSearchTables();
}

void FCompositingTextureLookupTable::Empty(const int32 KeepTags)
{
	if (KeepTags == 0x00)
	{
		LookupTable.Empty();
	}
	else 
	{
		for (auto It = LookupTable.CreateIterator(); It; ++It)
		{
			if ((It.Value().UsageTags & KeepTags) == 0x00)
			{
				It.RemoveCurrent();
			}
		}
	}
}

void FCompositingTextureLookupTable::ClearTaggedEntries(const int32 UsageTags, const bool bRemove)
{
	for (auto It = LookupTable.CreateIterator(); It; ++It)
	{
		if ((It.Value().UsageTags & UsageTags) != 0x00)
		{
			It.Value().Texture = nullptr;
			if (bRemove)
			{
				It.RemoveCurrent();
			}
		}
	}
}

void FCompositingTextureLookupTable::Remove(UTexture* Texture)
{
	for (auto It = LookupTable.CreateIterator(); It; ++It)
	{
		if (It.Value().Texture == Texture)
		{
			It.RemoveCurrent();
		}
	}
}

void FCompositingTextureLookupTable::LinkNestedSearchTable(FName KeyName, ICompositingTextureLookupTable* NestedLookupTable)
{
	LinkedSearchTables.Add(KeyName, NestedLookupTable);
}

void FCompositingTextureLookupTable::ClearLinkedSearchTables()
{
	LinkedSearchTables.Empty();
}

int32 FCompositingTextureLookupTable::FindUsageTags(FName LookupName)
{
	int32 UsageFlags = 0x00;
	if (const FTaggedTexture* TableItem = LookupTable.Find(LookupName))
	{
		UsageFlags = TableItem->UsageTags;
	}
	return UsageFlags;
}

bool FCompositingTextureLookupTable::FindNamedPassResult(FName LookupName, UTexture*& OutTexture) const
{
	return FindNamedPassResult(LookupName, /*bSearchLinkedTables =*/true, OutTexture);
}

bool FCompositingTextureLookupTable::FindNamedPassResult(FName LookupName, bool bSearchLinkedTables, UTexture*& OutTexture) const
{
	using namespace CompositingTextureLookupTable_Impl;

	const bool bIsAskingForPrePass = IsLookupNameAPrePassAlias(LookupName);
	if (bIsAskingForPrePass)
	{
		LookupName = GetAuthoratativePrePassLookupName();
	}

	bool bFound = false;
	if (const FTaggedTexture* TableItem = LookupTable.Find(LookupName))
	{
		bFound = true;
		OutTexture = TableItem->Texture;
	}
	else if (bIsAskingForPrePass)
	{
		bFound = true;
		OutTexture = nullptr;
	}
	else if (bSearchLinkedTables)
	{
		FString SearchStr = LookupName.ToString();

		for (auto& SubTable : LinkedSearchTables)
		{
			if (LookupName == SubTable.Key)
			{
				bFound = SubTable.Value->FindNamedPassResult(GetAuthoratativePrePassLookupName(), OutTexture);
				break;
			}
			else
			{
				FString SubTablePrefix = SubTable.Key.ToString() + TEXT('.');
				if (SearchStr.RemoveFromStart(SubTablePrefix))
				{
					bFound = SubTable.Value->FindNamedPassResult(*SearchStr, OutTexture);
					break;
				}
			}
		}
	}
	return bFound;
}
