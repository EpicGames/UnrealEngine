// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElements/PlayerViewportCompositingOutput.h"
#include "GameFramework/PlayerController.h"
#include "Kismet/GameplayStatics.h" // for GetPlayerController()
#include "Engine/LocalPlayer.h" // for ViewportClient
#include "Engine/GameViewportClient.h" // for EngineShowFlags
#include "ComposureUtils.h" // for SetEngineShowFlagsForPostprocessingOnly()
#include "ComposureInternals.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "SceneView.h"
#include "ComposureTonemapperPass.h" // for FComposureTonemapperUtils::ApplyTonemapperSettings()
#include "CompositingElements/CompositingElementPassUtils.h" // for GetTargetFormatFromPixelFormat()
#include "CompositingElements/CompositingElementTransforms.h" // for UCompositingTonemapPass
#include "Engine/Texture2D.h"
#include "UObject/ConstructorHelpers.h"

/* FPlayerViewportOverrideStack
 *****************************************************************************/

struct FPlayerViewportOverrideStack
{
public:
	void Push(APlayerController* TargetController);
	void Pop( APlayerController* TargetController);

private:
	struct FPlayerViewportOverride
	{
		FEngineShowFlags PreOverrideShowFlags;
		bool bOverridingShowFlags;
		int32 OverrideCount;
		bool bPreOverrideRenderPrimitives;

		FPlayerViewportOverride() 
			: PreOverrideShowFlags(ESFIM_Game), bOverridingShowFlags(false), OverrideCount(0), bPreOverrideRenderPrimitives(true) 
		{}
	};

	TMap<APlayerController*, FPlayerViewportOverride> ActiveOverrides;
};

void FPlayerViewportOverrideStack::Push(APlayerController* TargetController)
{
	FPlayerViewportOverride& ActiveOverride = ActiveOverrides.FindOrAdd(TargetController);
	if (ActiveOverride.OverrideCount == 0)
	{
		ActiveOverride.bPreOverrideRenderPrimitives = TargetController->bRenderPrimitiveComponents;
		TargetController->bRenderPrimitiveComponents = false;

		ULocalPlayer* Player = TargetController->GetLocalPlayer();
		if (Player && Player->ViewportClient)
		{
			FEngineShowFlags& PlayerShowFlags = Player->ViewportClient->EngineShowFlags;
			ActiveOverride.PreOverrideShowFlags = PlayerShowFlags;
			FComposureUtils::SetEngineShowFlagsForPostprocessingOnly(PlayerShowFlags);

			ActiveOverride.bOverridingShowFlags = true;
		}
		++ActiveOverride.OverrideCount;
	}
	else
	{
		++ActiveOverride.OverrideCount;
	}
}

void FPlayerViewportOverrideStack::Pop(APlayerController* TargetController)
{
	if (FPlayerViewportOverride* Override = ActiveOverrides.Find(TargetController))
	{
		if (--Override->OverrideCount <= 0)
		{
			ensure(Override->OverrideCount == 0); // make sure the count isn't getting off (less than zero)
			TargetController->bRenderPrimitiveComponents = Override->bPreOverrideRenderPrimitives;

			if (Override->bOverridingShowFlags)
			{
				ULocalPlayer* Player = TargetController->GetLocalPlayer();
				if (Player && Player->ViewportClient)
				{
					Player->ViewportClient->EngineShowFlags = Override->PreOverrideShowFlags;
				}
			}

			ActiveOverrides.Remove(TargetController);
		}
	}
}

namespace PlayerViewportCompDispatcher_Impl
{
	static const FName PlayerViewportMatInputName(TEXT("Input"));
	static FPlayerViewportOverrideStack OverrideStack;
}

/* UPlayerViewportCompositingOutput
 *****************************************************************************/

UPlayerViewportCompositingOutput::UPlayerViewportCompositingOutput()
{
	COMPOSURE_GET_MATERIAL(MaterialInterface, TonemapperBaseMat, "ReplaceTonemapper/", "ComposureReplaceTonemapperByTexture");
	COMPOSURE_GET_MATERIAL(MaterialInterface, PreTonemapBaseMat, "PassSetup/", "ComposureSimpleSetupMaterial");

	DefaultConverterClass = UCompositingTonemapPass::StaticClass();
}

UPlayerViewportCompositingOutput::~UPlayerViewportCompositingOutput()
{
	ClearViewportOverride();
}

void UPlayerViewportCompositingOutput::OnFrameBegin_Implementation(bool bCameraCutThisFrame)
{
	Super::OnFrameBegin_Implementation(bCameraCutThisFrame);

	if (ActiveOverrideIndex != PlayerIndex || !TargetedPlayerController.IsValid())
	{
		if (OverridePlayerCamera(PlayerIndex))
		{
			ViewportOverrideMID = GetBlendableMID();
		}
	}
}

void UPlayerViewportCompositingOutput::RelayOutput_Implementation(UTexture* RenderResult, UComposurePostProcessingPassProxy* PostProcessProxy)
{
#if WITH_EDITOR
	PreviewResult = RenderResult;
#endif

	if (ViewportOverrideMID)
	{
		UTexture* OutputImage = RenderResult;
		if (ColorConverter && !UseBuiltInColorConversion())
		{
			const FIntPoint RenderSize(RenderResult->GetSurfaceWidth(), RenderResult->GetSurfaceWidth());

			UTexture* ColorConvertResult = nullptr;
			if (UTextureRenderTarget2D* AsRenderTarget = Cast<UTextureRenderTarget2D>(RenderResult))
			{
				ColorConvertResult = ApplyColorTransform(RenderResult, PostProcessProxy, RenderSize, AsRenderTarget->RenderTargetFormat);
			}
			else
			{
				ETextureRenderTargetFormat RenderFormat = RTF_RGBA16f;
				if (UTexture2D* AsTex2D = Cast<UTexture2D>(RenderResult))
				{
					FCompositingElementPassUtils::GetTargetFormatFromPixelFormat(AsTex2D->GetPixelFormat(), RenderFormat);
				}
				ColorConvertResult = ApplyColorTransform(RenderResult, PostProcessProxy, RenderSize, RenderFormat);
			}

			if (ColorConvertResult)
			{
				OutputImage = ColorConvertResult;
			}
		}

		ViewportOverrideMID->SetTextureParameterValue(PlayerViewportCompDispatcher_Impl::PlayerViewportMatInputName, OutputImage);
	}
	else
	{
#if WITH_EDITOR
		// Run the color conversion for in-editor previewing sake
		PreviewResult = ApplyColorTransform(RenderResult, PostProcessProxy);
#endif 
	}
}

void UPlayerViewportCompositingOutput::Reset_Implementation()
{
	ClearViewportOverride();
	Super::Reset_Implementation();
}

void UPlayerViewportCompositingOutput::OverrideBlendableSettings(class FSceneView& View, float Weight) const
{
	// Clear any blendables that could have been set by post process volumes.
	View.FinalPostProcessSettings.BlendableManager = FBlendableManager();

	if (ViewportOverrideMID)
	{
		// Setup the post process material that dump the render target.
		ViewportOverrideMID->OverrideBlendableSettings(View, Weight);
	}
}

#if WITH_EDITOR
bool UPlayerViewportCompositingOutput::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPlayerViewportCompositingOutput, ApplyToneCurve))
	{
		//Note: When ColorConverter is null, a tonemap pass is automatically created (based on the DefaultConverterClass set above)
		return ColorConverter == nullptr || ColorConverter->GetClass() == UCompositingTonemapPass::StaticClass();
	}

	return true;
}
#endif

bool UPlayerViewportCompositingOutput::OverridePlayerCamera(int32 InPlayerIndex)
{
	ClearViewportOverride();

	TargetedPlayerController = UGameplayStatics::GetPlayerController(/*WorldContextObject =*/this, InPlayerIndex);
	if (TargetedPlayerController.IsValid())
	{
		PlayerViewportCompDispatcher_Impl::OverrideStack.Push(TargetedPlayerController.Get());

		APlayerCameraManager* PlayerCamManager = TargetedPlayerController->PlayerCameraManager;
		if (PlayerCamManager)
		{
			UCameraModifier* NewModifier = PlayerCamManager->AddNewCameraModifier(UPlayerCompOutputCameraModifier::StaticClass());
			ActiveCamModifier = CastChecked<UPlayerCompOutputCameraModifier>(NewModifier);
			ActiveCamModifier->SetOwner(this);
		}

		ActiveOverrideIndex = InPlayerIndex;
	}
	return TargetedPlayerController.IsValid();
}

void UPlayerViewportCompositingOutput::ClearViewportOverride()
{
	if (TargetedPlayerController.IsValid())
	{
		APlayerCameraManager* PlayerCamManager = TargetedPlayerController->PlayerCameraManager;
		if (PlayerCamManager)
		{
			PlayerCamManager->RemoveCameraModifier(ActiveCamModifier);
		}

		PlayerViewportCompDispatcher_Impl::OverrideStack.Pop(TargetedPlayerController.Get());
	}

	ActiveCamModifier = nullptr;
	TargetedPlayerController.Reset();
	ActiveOverrideIndex = INDEX_NONE;
}

UMaterialInstanceDynamic* UPlayerViewportCompositingOutput::GetBlendableMID()
{
	UMaterialInterface* BaseMat = PreTonemapBaseMat;
	if (!UseBuiltInColorConversion())
	{
		BaseMat = TonemapperBaseMat;
	}

	if (ViewportOverrideMID == nullptr || ViewportOverrideMID->GetBaseMaterial() != BaseMat)
	{
		ViewportOverrideMID = UMaterialInstanceDynamic::Create(BaseMat, /*Outer =*/this);
	}
	return ViewportOverrideMID;
}

bool UPlayerViewportCompositingOutput::UseBuiltInColorConversion() const
{
	// if the external pass would just do tonemapping for us,
	// it is more efficient to run the tonemapping internally, 
	// as part of the player's post-process pipeline instead (saves on a render target, etc.)
	return ColorConverter && ColorConverter->GetClass() == UCompositingTonemapPass::StaticClass();
}

/* UPlayerCompOutputCameraModifier
 *****************************************************************************/

void UPlayerCompOutputCameraModifier::SetOwner(UPlayerViewportCompositingOutput* InOwner)
{
	Owner = InOwner;
	// @TODO: handle ownership transfer?
}

bool UPlayerCompOutputCameraModifier::ModifyCamera(float /*DeltaTime*/, FMinimalViewInfo& InOutPOV)
{
	if (Owner)
	{ 
		if (Owner->UseBuiltInColorConversion())
		{
			UCompositingTonemapPass* Tonemapper = CastChecked<UCompositingTonemapPass>(Owner->ColorConverter);

			FComposureTonemapperUtils::ApplyTonemapperSettings(
				Tonemapper->ColorGradingSettings,
				Tonemapper->FilmStockSettings,
				Tonemapper->ChromaticAberration,
				InOutPOV.PostProcessSettings
			);
		}

		if (!Owner->ApplyToneCurve)
		{
			InOutPOV.PostProcessSettings.bOverride_ToneCurveAmount = true;
			InOutPOV.PostProcessSettings.ToneCurveAmount = 0;

			InOutPOV.PostProcessSettings.bOverride_ExpandGamut = true;
			InOutPOV.PostProcessSettings.ExpandGamut = 0;
		}

		InOutPOV.PostProcessSettings.AddBlendable(Owner, /*Weight =*/1.0f);

		return true;
	}
	return false;
}
