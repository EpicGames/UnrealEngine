// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureBlueprintLibrary.h"

#include "CineCameraComponent.h"
#include "CompElementEditorModule.h"
#include "Components/SceneCaptureComponent2D.h"
#include "CompositingElement.h"
#include "ComposurePlayerCompositingTarget.h"
#include "ComposurePostMoves.h"
#include "ComposureUtils.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/PlayerController.h"
#include "ICompElementManager.h"
#include "Modules/ModuleManager.h"
#include "Slate/SceneViewport.h"
#include "UObject/Package.h"

UComposureBlueprintLibrary::UComposureBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ }


// static
UComposurePlayerCompositingTarget* UComposureBlueprintLibrary::CreatePlayerCompositingTarget(UObject* WorldContextObject)
{
	UObject* Outer = WorldContextObject ? WorldContextObject : GetTransientPackage();
	return NewObject< UComposurePlayerCompositingTarget>(Outer);
}

// static
void UComposureBlueprintLibrary::GetProjectionMatrixFromPostMoveSettings(
	const FComposurePostMoveSettings& PostMoveSettings, float HorizontalFOVAngle, float AspectRatio, FMatrix& ProjectionMatrix)
{
	ProjectionMatrix = PostMoveSettings.GetProjectionMatrix(HorizontalFOVAngle, AspectRatio);
}

// static
void UComposureBlueprintLibrary::GetCroppingUVTransformationMatrixFromPostMoveSettings(
	const FComposurePostMoveSettings& PostMoveSettings, float AspectRatio,
	FMatrix& CropingUVTransformationMatrix, FMatrix& UncropingUVTransformationMatrix)
{
	PostMoveSettings.GetCroppingUVTransformationMatrix(AspectRatio, &CropingUVTransformationMatrix, &UncropingUVTransformationMatrix);
}

// static
void UComposureBlueprintLibrary::GetRedGreenUVFactorsFromChromaticAberration(
	float ChromaticAberrationAmount, FVector2D& RedGreenUVFactors)
{
	RedGreenUVFactors = FComposureUtils::GetRedGreenUVFactorsFromChromaticAberration(
		FMath::Clamp(ChromaticAberrationAmount, 0.f, 1.f));
}

//static
void UComposureBlueprintLibrary::GetPlayerDisplayGamma(const APlayerCameraManager* PlayerCameraManager, float& DisplayGamma)
{
	DisplayGamma = 0;
	if (!PlayerCameraManager)
	{
		return;
	}

	UGameViewportClient* ViewportClient = PlayerCameraManager->PCOwner->GetLocalPlayer()->ViewportClient;
	if (!ViewportClient)
	{
		return;
	}

	FSceneViewport* SceneViewport = ViewportClient->GetGameViewport();

	DisplayGamma = SceneViewport ? SceneViewport->GetDisplayGamma() : 0.0;
}

void UComposureBlueprintLibrary::CopyCameraSettingsToSceneCapture(UCameraComponent* Src, USceneCaptureComponent2D* Dst, float OriginalFocalLength, float OverscanFactor)
{
	if (Src && Dst)
	{
		Dst->SetWorldLocationAndRotation(Src->GetComponentLocation(), Src->GetComponentRotation());

		FMinimalViewInfo CameraViewInfo;
		Src->GetCameraView(/*DeltaTime =*/0.0f, CameraViewInfo);

		// Use the input overscan factor to augment the destination component's FOV angle
		// Note: The math relies on the filmback and focal length of the input camera component, which necessitates that it be a CineCameraComponet
		if (UCineCameraComponent* SrcCineCameraComponent = Cast<UCineCameraComponent>(Src))
		{ 			
			// Guard against divide-by-zero
			if (SrcCineCameraComponent->CurrentFocalLength <= 0.0f)
			{
				Dst->FOVAngle = 0.0f;
			}
			else
			{
				const float OverscanSensorWidth = SrcCineCameraComponent->Filmback.SensorWidth * SrcCineCameraComponent->LensSettings.SqueezeFactor * OverscanFactor;
				Dst->FOVAngle = FMath::RadiansToDegrees(2.0f * FMath::Atan(OverscanSensorWidth / (2.0f * OriginalFocalLength)));
			}
		}
		else
		{
			Dst->FOVAngle = Src->FieldOfView;
		}

		const FPostProcessSettings& SrcPPSettings = CameraViewInfo.PostProcessSettings;
		FPostProcessSettings& DstPPSettings = Dst->PostProcessSettings;

		FWeightedBlendables DstWeightedBlendables = DstPPSettings.WeightedBlendables;

		// Copy all of the post processing settings
		DstPPSettings = SrcPPSettings;

		// But restore the original blendables
		DstPPSettings.WeightedBlendables = DstWeightedBlendables;
	}
}

ACompositingElement* UComposureBlueprintLibrary::CreateComposureElement(const FName CompName, TSubclassOf<ACompositingElement> ClassType, AActor* LevelContext)
{
	if (ClassType)
	{
		ICompElementEditorModule* EditorModule = FModuleManager::Get().GetModulePtr<ICompElementEditorModule>("ComposureLayersEditor");
		TSharedPtr<ICompElementManager> CompElementManager;
		if (EditorModule && (CompElementManager = EditorModule->GetCompElementManager()).IsValid())
		{
			TWeakObjectPtr<ACompositingElement> CreatedElement = CompElementManager->CreateElement(CompName, ClassType, LevelContext);
			CompElementManager->OnCreateNewElement(CreatedElement.Get());
			return CreatedElement.Get();
		}
	}
	return nullptr;
}

ACompositingElement* UComposureBlueprintLibrary::GetComposureElement(const FName ElementName)
{
	ICompElementEditorModule* EditorModule = FModuleManager::Get().GetModulePtr<ICompElementEditorModule>("ComposureLayersEditor");
	TSharedPtr<ICompElementManager> CompElementManager;
	if (EditorModule && (CompElementManager = EditorModule->GetCompElementManager()).IsValid())
	{
		return CompElementManager->GetElement(ElementName).Get();
	}
	return nullptr;
}


void UComposureBlueprintLibrary::DeleteComposureElementAndChildren(const FName ElementToDelete)
{
	ICompElementEditorModule* EditorModule = FModuleManager::Get().GetModulePtr<ICompElementEditorModule>("ComposureLayersEditor");
	TSharedPtr<ICompElementManager> CompElementManager;
	if (EditorModule && (CompElementManager = EditorModule->GetCompElementManager()).IsValid())
	{
		CompElementManager->DeleteElementAndChildren(ElementToDelete,false);
	}
}

bool UComposureBlueprintLibrary::RenameComposureElement(const FName OriginalName, const FName NewCompName)
{
	ICompElementEditorModule* EditorModule = FModuleManager::Get().GetModulePtr<ICompElementEditorModule>("ComposureLayersEditor");
	TSharedPtr<ICompElementManager> CompElementManager;
	if (EditorModule && (CompElementManager = EditorModule->GetCompElementManager()).IsValid())
	{
		return CompElementManager->RenameElement(OriginalName, NewCompName);
	}
	return false;
}

bool UComposureBlueprintLibrary::AttachComposureElement(const FName ParentName, const FName ChildName)
{
	ICompElementEditorModule* EditorModule = FModuleManager::Get().GetModulePtr<ICompElementEditorModule>("ComposureLayersEditor");
	TSharedPtr<ICompElementManager> CompElementManager;
	if (EditorModule && (CompElementManager = EditorModule->GetCompElementManager()).IsValid())
	{
		return CompElementManager->AttachCompElement(ParentName, ChildName);
	}
	return false;
}

bool UComposureBlueprintLibrary::IsComposureElementDrawing(ACompositingElement* CompElement)
{
	bool bIsDrawing = false;
	if (CompElement)
	{
		ICompElementEditorModule* EditorModule = FModuleManager::Get().GetModulePtr<ICompElementEditorModule>("ComposureLayersEditor");
		TSharedPtr<ICompElementManager> CompElementManager;
		if (EditorModule && (CompElementManager = EditorModule->GetCompElementManager()).IsValid())
		{
			bIsDrawing = CompElementManager->IsDrawing(CompElement);
		}
	}
	return bIsDrawing;
}

void UComposureBlueprintLibrary::RequestRedrawComposureViewport()
{
	ICompElementEditorModule* EditorModule = FModuleManager::Get().GetModulePtr<ICompElementEditorModule>("ComposureLayersEditor");
	TSharedPtr<ICompElementManager> CompElementManager;
	if (EditorModule && (CompElementManager = EditorModule->GetCompElementManager()).IsValid())
	{
		CompElementManager->RequestRedraw();
	}
}

void UComposureBlueprintLibrary::RefreshComposureElementList()
{
	ICompElementEditorModule* EditorModule = FModuleManager::Get().GetModulePtr<ICompElementEditorModule>("ComposureLayersEditor");
	TSharedPtr<ICompElementManager> CompElementManager;
	if (EditorModule && (CompElementManager = EditorModule->GetCompElementManager()).IsValid())
	{
		CompElementManager->RefreshElementsList();
	}
}
