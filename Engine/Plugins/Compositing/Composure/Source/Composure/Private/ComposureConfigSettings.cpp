// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureConfigSettings.h"
#include "Engine/Texture.h"

UComposureGameSettings::UComposureGameSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bSceneCapWarnOfMissingCam(true)
	, FallbackCompositingTexture(TEXT("/Engine/Functions/Engine_MaterialFunctions02/PivotPainter2/Black_1x1_EXR_Texture.Black_1x1_EXR_Texture"))
{
	IConsoleManager& CVarManager = IConsoleManager::Get();

	CVarManager.RegisterConsoleVariableRef(
		TEXT("r.Composure.CompositingElements.Editor.WarnWhenSceneCaptureIsMissingCamera"),
		bSceneCapWarnOfMissingCam,
		TEXT("By default, scene capture (CG) elements rely on a camera to position themselves.\n")
		TEXT("To catch when one isn't set up, the editor displays a warning image.\n")
		TEXT("Disable this CVar to allow the capture from the element's position & orientation."),
		ECVF_Default);
}

UTexture* UComposureGameSettings::GetFallbackCompositingTexture()
{
	UComposureGameSettings* Settings = GetMutableDefault<UComposureGameSettings>();
	if (!Settings->FallbackCompositingTextureObj)
	{
		Settings->FallbackCompositingTextureObj = Cast<UTexture>(Settings->FallbackCompositingTexture.TryLoad());
	}
	return Settings->FallbackCompositingTextureObj;
}
