// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComposureConfigSettings.generated.h"

class UTexture;

UCLASS(config=Game)
class UComposureGameSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(Config, BlueprintReadOnly, Category="Composure|Media")
	FSoftObjectPath StaticVideoPlateDebugImage;

	UPROPERTY(Config, BlueprintReadOnly, Category="Composure|Editor")
	bool bSceneCapWarnOfMissingCam;

	UPROPERTY(Config, BlueprintReadOnly, Category="Composure")
	FSoftObjectPath FallbackCompositingTexture;
	static UTexture* GetFallbackCompositingTexture();
	
private:
	UPROPERTY(Transient)
	TObjectPtr<UTexture> FallbackCompositingTextureObj;
};
