// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FComposureCustomVersion::GUID(0x4cef9221, 0x470ed43a, 0x7e603d8c, 0x16995726);

// Register the custom version with core
FCustomVersionRegistration GRegisterComposureCustomVersion(FComposureCustomVersion::GUID, FComposureCustomVersion::LatestVersion, TEXT("ComposureVer"));
