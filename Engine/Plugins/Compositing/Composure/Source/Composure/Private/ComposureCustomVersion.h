// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom serialization version for all packages containing Paper2D asset types
struct FComposureCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,

		NewPassSystem,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	static const FGuid GUID;

private:
	FComposureCustomVersion() {}
};
