// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"


// Macro to create a composure material and set it to <DestMemberName>.
#define COMPOSURE_GET_MATERIAL(MaterialType,DestMemberName,MaterialDirName,MaterialFileName) \
	static ConstructorHelpers::FObjectFinder<U##MaterialType> G##DestMemberName##Material( \
		TEXT(#MaterialType "'/Composure/Materials/" MaterialDirName MaterialFileName "." MaterialFileName "'")); \
	DestMemberName = G##DestMemberName##Material.Object


// Macro to create a composure dynamic material instance and set it to <DestMemberName>.
#define COMPOSURE_CREATE_DYMAMIC_MATERIAL(MaterialType,DestMemberName,MaterialDirName,MaterialFileName) \
	static ConstructorHelpers::FObjectFinder<U##MaterialType> G##DestMemberName##Material( \
		TEXT(#MaterialType "'/Composure/Materials/" MaterialDirName MaterialFileName "." MaterialFileName "'")); \
	DestMemberName = UMaterialInstanceDynamic::Create(G##DestMemberName##Material.Object, this, TEXT(#DestMemberName))

#define COMPOSURE_GET_TEXTURE(TextureType,DestMemberName,TextureDirName,TextureFileName) \
	static ConstructorHelpers::FObjectFinder<U##TextureType> G##DestMemberName##Texture( \
		TEXT(#TextureType "'/Composure/Textures/" TextureDirName TextureFileName "." TextureFileName "'")); \
	DestMemberName = G##DestMemberName##Texture.Object


DECLARE_LOG_CATEGORY_EXTERN(Composure, Log, All);
