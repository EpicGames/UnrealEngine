// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureLensBloomPass.h"
#include "ComposurePostProcessBlendable.h"

#include "Components/SceneCaptureComponent2D.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ComposureInternals.h"
#include "ComposureUtils.h"
#include "UObject/ConstructorHelpers.h"

/* ComposureLensBloomPass_Impl
 *****************************************************************************/

namespace ComposureLensBloomPass_Impl
{
	static void ApplyBloomSettings(const FLensBloomSettings& BloomSettings, USceneCaptureComponent2D* SceneCapture, UMaterialInstanceDynamic* TonemapperMID, FName IntensityParamName);
}

static void ComposureLensBloomPass_Impl::ApplyBloomSettings(const FLensBloomSettings& BloomSettings, USceneCaptureComponent2D* SceneCapture, UMaterialInstanceDynamic* TonemapperMID, FName IntensityParamName)
{
	// Exports the settings to the scene capture's post process settings.
	BloomSettings.ExportToPostProcessSettings(&SceneCapture->PostProcessSettings);

	if (TonemapperMID)
	{
		TonemapperMID->SetScalarParameterValue(IntensityParamName, SceneCapture->PostProcessSettings.BloomIntensity);
	}

	// Enables bloom.
	SceneCapture->ShowFlags.Bloom = true;
}

/* UComposureLensBloomPass
 *****************************************************************************/

UComposureLensBloomPass::UComposureLensBloomPass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Replace the tonemapper with a post process material that outputs bloom only.
	COMPOSURE_CREATE_DYMAMIC_MATERIAL(Material, TonemapperReplacingMID, "ReplaceTonemapper/", "ComposureReplaceTonemapperComposeBloom");
	TonemapperReplacement = TonemapperReplacingMID;
}

void UComposureLensBloomPass::SetTonemapperReplacingMaterial(UMaterialInstanceDynamic* Material)
{
	UMaterial* Base = Material->GetBaseMaterial();

	if (Base->MaterialDomain == MD_PostProcess &&
		Base->BlendableLocation == BL_ReplacingTonemapper)
	{
		TonemapperReplacement = TonemapperReplacingMID = Material;
	}
}

void UComposureLensBloomPass::BloomToRenderTarget()
{
	// Disables as much stuf as possible using showflags. 
	FComposureUtils::SetEngineShowFlagsForPostprocessingOnly(SceneCapture->ShowFlags);

	ComposureLensBloomPass_Impl::ApplyBloomSettings(Settings, SceneCapture, TonemapperReplacingMID, TEXT("BloomIntensity"));

	// Adds the blendable to have programmatic control of FSceneView::FinalPostProcessSettings
	// in  UComposurePostProcessPass::OverrideBlendableSettings().
	SceneCapture->PostProcessSettings.AddBlendable(BlendableInterface, 1.0);

	SceneCapture->ProfilingEventName = TEXT("ComposureLensBloomPass");

	// OverrideBlendableSettings() will do nothing (see UMaterialInterface::OverrideBlendableSettings) 
	// with these materials unless there is a ViewState from the capture component (see USceneCaptureComponent::GetViewState)
	TGuardValue<bool> ViewStateGuard(SceneCapture->bAlwaysPersistRenderingState, true);

	// Update the render target output.
	SceneCapture->CaptureScene();
}

/* UComposureLensBloomPassPolicy
 *****************************************************************************/

UComposureLensBloomPassPolicy::UComposureLensBloomPassPolicy()
	: BloomIntensityParamName(TEXT("BloomIntensity"))
{
	COMPOSURE_GET_MATERIAL(Material, ReplacementMaterial, "ReplaceTonemapper/", "ComposureReplaceTonemapperComposeBloom");
}

void UComposureLensBloomPassPolicy::SetupPostProcess_Implementation(USceneCaptureComponent2D* SceneCapture, UMaterialInterface*& OutTonemapperOverride)
{
	if (ReplacementMaterial)
	{
		if (!TonemapperReplacmentMID || TonemapperReplacmentMID->GetBaseMaterial() != ReplacementMaterial)
		{
			TonemapperReplacmentMID = UMaterialInstanceDynamic::Create(ReplacementMaterial, this);
		}
	}
	else
	{
		TonemapperReplacmentMID = nullptr;
	}
	OutTonemapperOverride = TonemapperReplacmentMID;

	ComposureLensBloomPass_Impl::ApplyBloomSettings(Settings, SceneCapture, TonemapperReplacmentMID, BloomIntensityParamName);
}
