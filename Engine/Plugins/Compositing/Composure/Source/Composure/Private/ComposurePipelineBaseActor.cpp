// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposurePipelineBaseActor.h"
#include "ComposureViewExtension.h"
#include "LevelUtils.h"

#if WITH_EDITOR
#include "Editor.h"

static TAutoConsoleVariable<int32> CVarSuspendEditorInstancesWithPIE(
	TEXT("r.Composure.PipelineActors.SuspendEditorInstancesWithPIE"),
	1,
	TEXT("This suspends composure editor rendering when you're in PIE. ")
	TEXT("The PIE instances will still render, just their Editor world counterparts will temporarily stop.\n")
	TEXT("If disabled, both insances (the Editor's and their corespponding copies in PIE) will render at the same time - taxing resources, and slowing down the renderer."));
#endif

AComposurePipelineBaseActor::AComposurePipelineBaseActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAutoRun(true)
#if WITH_EDITOR
	, bRunInEditor(true)
#endif
{
	PrimaryActorTick.bCanEverTick = true;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		ViewExtension = FSceneViewExtensions::NewExtension<FComposureViewExtension>(this);
	}
}

#if WITH_EDITOR
void AComposurePipelineBaseActor::RerunConstructionScripts()
{
	if (GEditor && GEditor->bIsSimulatingInEditor)
	{
		// Don't reconstruct blueprints if simulating so that keyframe in sequencer doesn't clobber the
		// pipeline state.
		return;
	}

	Super::RerunConstructionScripts();
}
#endif

bool AComposurePipelineBaseActor::IsActivelyRunning_Implementation() const
{
	UWorld* MyWorld = GetWorld();
#if !WITH_EDITOR
	bool bActive = true;
#else
	bool bActive = !IsAutoRunSuspended();

	if (MyWorld && MyWorld->WorldType == EWorldType::Editor)
	{
		bActive &= bRunInEditor;
	}
	else
#endif // WITH_EDITOR
	{
		bActive &= MyWorld && MyWorld->IsGameWorld();
	}
	bActive &= bAutoRun && !HasAnyFlags(RF_ClassDefaultObject);

	if (ULevel* MyLevel = GetLevel())
	{
		bActive &= FLevelUtils::IsLevelVisible(MyLevel) && FLevelUtils::IsLevelLoaded(MyLevel);
	}

	return bActive;
}

#if WITH_EDITOR
bool AComposurePipelineBaseActor::IsAutoRunSuspended() const
{
	UWorld* MyWorld = GetWorld();
	const bool bIsEditorInstance = (MyWorld && MyWorld->WorldType == EWorldType::Editor);

	const bool bIsPIEing = GEditor && (GEditor->PlayWorld != nullptr);
	return bIsEditorInstance && bIsPIEing && CVarSuspendEditorInstancesWithPIE.GetValueOnGameThread();
}
#endif


void AComposurePipelineBaseActor::EnqueueRendering_Implementation(bool /*bCameraCutThisFrame*/)
{

}
