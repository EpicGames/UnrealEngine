// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraModifier.h"
#include "Engine/BlendableInterface.h"
#include "ComposurePlayerCompositingCameraModifier.generated.h"

struct FMinimalViewInfo;

class IComposurePlayerCompositingInterface;

/**
 * Private camera manager for  UComposurePlayerCompositingTarget.
 */
UCLASS(NotBlueprintType)
class UComposurePlayerCompositingCameraModifier
	: public UCameraModifier
	, public IBlendableInterface
{
	GENERATED_UCLASS_BODY()

public:

	// Begins UCameraModifier
	bool ModifyCamera(float DeltaTime, FMinimalViewInfo& InOutPOV) override;
	// Ends UCameraModifier

	// Begins IBlendableInterface
	void OverrideBlendableSettings(class FSceneView& View, float Weight) const override;
	// Ends IBlendableInterface

private:
	// Current player camera manager the target is bind on.
	UPROPERTY(Transient)
 	TScriptInterface<IComposurePlayerCompositingInterface> Target;

	friend class UComposurePlayerCompositingTarget;
	friend class UComposureCompositingTargetComponent;
};
