// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposurePlayerCompositingInterface.h"

UComposurePlayerCompositingInterface::UComposurePlayerCompositingInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}