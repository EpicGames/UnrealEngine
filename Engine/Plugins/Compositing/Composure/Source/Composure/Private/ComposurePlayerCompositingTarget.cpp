// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposurePlayerCompositingTarget.h"
#include "ComposurePlayerCompositingCameraModifier.h"

#include "EditorSupport/WeakUInterfacePtr.h"
#include "GameFramework/PlayerController.h"
#include "Engine/GameViewportClient.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/LocalPlayer.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "SceneView.h"

#include "ComposureUtils.h"
#include "ComposureInternals.h"
#include "UObject/ConstructorHelpers.h"

UComposurePlayerCompositingCameraModifier::UComposurePlayerCompositingCameraModifier(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Target(nullptr)
{ }

bool UComposurePlayerCompositingCameraModifier::ModifyCamera(float DeltaTime, FMinimalViewInfo& InOutPOV)
{
	check(Target);

	// Adds itself to have programmatic control of FSceneView::FinalPostProcessSettings
	// in  UComposurePlayerCompositingTarget::OverrideBlendableSettings().
	InOutPOV.PostProcessSettings.AddBlendable(this, /* Weight = */ 1.0f);
	return true;
}

void UComposurePlayerCompositingCameraModifier::OverrideBlendableSettings(FSceneView& View, float Weight) const
{
	check(Target);

	// Forward call to the  UComposurePlayerCompositingTarget.
	Target->OverrideBlendableSettings(View, Weight);
}



UComposurePlayerCompositingTarget::UComposurePlayerCompositingTarget( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
	, PlayerCameraManager(nullptr)
	, PlayerCameraModifier(nullptr)
	, EngineShowFlagsBackup(ESFIM_Game)
{
	COMPOSURE_CREATE_DYMAMIC_MATERIAL(Material, ReplaceTonemapperMID, "ReplaceTonemapper/", "ComposureReplaceTonemapperByTexture");
}

UComposurePlayerCompositingTarget::~UComposurePlayerCompositingTarget()
{
	check(!PlayerCameraManager);
}

APlayerCameraManager* UComposurePlayerCompositingTarget::SetPlayerCameraManager(APlayerCameraManager* InPlayerCameraManager)
{
	if (InPlayerCameraManager == PlayerCameraManager)
	{
		return InPlayerCameraManager;
	}
	else if (PlayerCameraManager)
	{
		// Remove the camera modifier from the camera manager.
		check(PlayerCameraModifier);
		PlayerCameraManager->RemoveCameraModifier(PlayerCameraModifier);
		PlayerCameraModifier = nullptr;

		// Resume rendering any components.
		PlayerCameraManager->PCOwner->bRenderPrimitiveComponents = true;

		// Restore local player's showflags.
		UGameViewportClient* ViewportClient = PlayerCameraManager->PCOwner->GetLocalPlayer()->ViewportClient;
		if (ViewportClient)
		{
			ViewportClient->EngineShowFlags = EngineShowFlagsBackup;
		}
	}

	PlayerCameraManager = InPlayerCameraManager;

	if (PlayerCameraManager)
	{
		// Stop rendering any component.
		check(PlayerCameraManager->PCOwner);
		PlayerCameraManager->PCOwner->bRenderPrimitiveComponents = false;

		// Setup camera modifier to the camera manager.
		check(!PlayerCameraModifier);
		PlayerCameraModifier = Cast<UComposurePlayerCompositingCameraModifier>(
			PlayerCameraManager->AddNewCameraModifier(UComposurePlayerCompositingCameraModifier::StaticClass()));
		PlayerCameraModifier->Target = this;

		// Setup local player's showflags.
		FEngineShowFlags& EngineShowFlags = PlayerCameraManager->PCOwner->GetLocalPlayer()->ViewportClient->EngineShowFlags;
		EngineShowFlagsBackup = EngineShowFlags;
		FComposureUtils::SetEngineShowFlagsForPostprocessingOnly(EngineShowFlags);
	}

	return InPlayerCameraManager;
}

void UComposurePlayerCompositingTarget::SetRenderTarget(UTextureRenderTarget2D* RenderTarget)
{
	ReplaceTonemapperMID->SetTextureParameterValue(TEXT("Input"), RenderTarget);
}

void UComposurePlayerCompositingTarget::FinishDestroy()
{
	SetPlayerCameraManager(nullptr);

	Super::FinishDestroy();
}

void UComposurePlayerCompositingTarget::OverrideBlendableSettings(class FSceneView& View, float Weight) const
{
	// Clear any blendables that could have been set by post process volumes.
	View.FinalPostProcessSettings.BlendableManager = FBlendableManager();

	// Setup the post process material that dump the render target.
	ReplaceTonemapperMID->OverrideBlendableSettings(View, Weight);
}

/* UComposureCompositingTargetComponent
 *****************************************************************************/

#if WITH_EDITOR
#include "EditorSupport/ICompositingEditor.h"
#include "Engine/Blueprint.h"
#endif

UComposureCompositingTargetComponent::UComposureCompositingTargetComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = true;

#if WITH_EDITOR
	COMPOSURE_GET_TEXTURE(Texture, CompilerErrImage, "Debug/", "T_CompilerError");
#endif
}

void UComposureCompositingTargetComponent::SetDisplayTexture(UTexture* InDisplayTexture)
{
	DisplayTexture = InDisplayTexture;
}

#if WITH_EDITOR
bool UComposureCompositingTargetComponent::IsPreviewing() const
{
	ensure(PreviewCount >= 0);
	return PreviewCount > 0;
}

bool UComposureCompositingTargetComponent::GetEditorPreviewInfo(float /*DeltaTime*/, FMinimalViewInfo& ViewOut)
{
	if (DisplayTexture != nullptr)
	{
		ViewOut.AspectRatio = DisplayTexture->GetSurfaceWidth() / DisplayTexture->GetSurfaceHeight();
	}
	ViewOut.bConstrainAspectRatio = true;

	return true;
}

TSharedPtr<SWidget> UComposureCompositingTargetComponent::GetCustomEditorPreviewWidget()
{
	TSharedPtr<SWidget> PreviewWidget;
	if (ICompositingEditor* CompositingEditor = ICompositingEditor::Get())
	{
		PreviewWidget = CompositingEditor->ConstructCompositingPreviewPane(TWeakUIntrfacePtr<ICompEditorImagePreviewInterface>(this));
	}
	return PreviewWidget;
}

void UComposureCompositingTargetComponent::OnBeginPreview()
{
	++PreviewCount;
}

UTexture* UComposureCompositingTargetComponent::GetEditorPreviewImage()
{
	UTexture* PreviewImage = DisplayTexture; 

	bool bActorHasCompileError = false;
	if (AActor* MyOwner = GetOwner())
	{
		UClass* ActorClass = MyOwner->GetClass();
		if (ActorClass && ActorClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(ActorClass->ClassGeneratedBy))
			{
				if ((bHasCompilerError && Blueprint->Status == EBlueprintStatus::BS_Dirty) || 
					(Blueprint->Status == EBlueprintStatus::BS_Error || Blueprint->Status == EBlueprintStatus::BS_Unknown))
				{
					PreviewImage = CompilerErrImage;
					bActorHasCompileError = true;
				}
			}
		}
	}
	bHasCompilerError = bActorHasCompileError;

	return PreviewImage;
}

void UComposureCompositingTargetComponent::OnEndPreview()
{
	--PreviewCount;
}
#endif // WITH_EDITOR
