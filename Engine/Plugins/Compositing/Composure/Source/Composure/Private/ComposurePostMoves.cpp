// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposurePostMoves.h"
#include "Math/PerspectiveMatrix.h"


FMatrix FComposurePostMoveSettings::GetProjectionMatrix(float HorizontalFOVAngle, float AspectRatio) const
{
	float HalfFOV = 0.5 * FMath::DegreesToRadians(HorizontalFOVAngle);

	FMatrix OriginalProjectionMatrix = FReversedZPerspectiveMatrix(
		HalfFOV,
		HalfFOV,
		/* XAxisMultiplier = */ 1.0,
		/* YAxisMultiplier = */ AspectRatio,
		GNearClippingPlane,
		GNearClippingPlane);

	FVector2D NormalizedViewRect = FMath::Tan(HalfFOV) * FVector2D(1, 1 / AspectRatio);
	FVector2D NormalizedPostMoveTranslation = 2.0 * NormalizedViewRect * Translation;
	FVector2D NormalizedPivot = NormalizedViewRect * (Pivot - 0.5) * FVector2D(2, 2);

	FMatrix ScaleMatrix(
		FPlane(Scale, 0, 0, 0),
		FPlane(0, Scale, 0, 0),
		FPlane(0, 0, 1, 0),
		FPlane(0, 0, 0, 1));

	FMatrix PreRotationMatrix(
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(-NormalizedPivot.X, -NormalizedPivot.Y, 1, 0),
		FPlane(0, 0, 0, 1));

	float Rotation = FMath::DegreesToRadians(RotationAngle);
	FMatrix RotationMatrix(
		FPlane(FMath::Cos(Rotation), FMath::Sin(Rotation), 0, 0),
		FPlane(-FMath::Sin(Rotation), FMath::Cos(Rotation), 0, 0),
		FPlane(0, 0, 1, 0),
		FPlane(0, 0, 0, 1));

	FMatrix PostRotationMatrix(
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(NormalizedPivot.X, NormalizedPivot.Y, 1, 0),
		FPlane(0, 0, 0, 1));

	FMatrix TranslateMatrix(
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(NormalizedPostMoveTranslation.X, NormalizedPostMoveTranslation.Y, 1, 0),
		FPlane(0, 0, 0, 1));

	return
		PreRotationMatrix *
		ScaleMatrix *
		RotationMatrix *
		PostRotationMatrix *
		TranslateMatrix *
		OriginalProjectionMatrix;
}

void FComposurePostMoveSettings::GetCroppingUVTransformationMatrix(
	float AspectRatio,
	FMatrix* OutCropingUVTransformationMatrix,
	FMatrix* OutUncropingUVTransformationMatrix) const
{
	FVector2D UVSpacePivot = FVector2D(Pivot.X, 1.0 - Pivot.Y);
	FMatrix ScaleMatrix(
		FPlane(1 / Scale, 0, 0, 0),
		FPlane(0, 1 / Scale, 0, 0),
		FPlane(0, 0, 1, 0),
		FPlane(0, 0, 0, 1));

	FMatrix PreRotationMatrix(
		FPlane(1, 0, 0, 0),
		FPlane(0, AspectRatio, 0, 0),
		FPlane(0, 0, 1, 0),
		FPlane(UVSpacePivot.X, UVSpacePivot.Y, 0, 1));

	float Rotation = FMath::DegreesToRadians(RotationAngle);
	FMatrix RotationMatrix(
		FPlane(FMath::Cos(Rotation), FMath::Sin(Rotation), 0, 0),
		FPlane(-FMath::Sin(Rotation), FMath::Cos(Rotation), 0, 0),
		FPlane(0, 0, 1, 0),
		FPlane(0, 0, 0, 1));

	FMatrix PostRotationMatrix(
		FPlane(1, 0, 0, 0),
		FPlane(0, 1 / AspectRatio, 0, 0),
		FPlane(0, 0, 1, 0),
		FPlane(-UVSpacePivot.X, -UVSpacePivot.Y / AspectRatio, 0, 1));

	FMatrix TranslateMatrix(
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 1, 0),
		FPlane(-Translation.X, Translation.Y, 0, 1));

	FMatrix InverseScaleMatrix(
		FPlane(Scale, 0, 0, 0),
		FPlane(0, Scale, 0, 0),
		FPlane(0, 0, 1, 0),
		FPlane(0, 0, 0, 1));

	FMatrix InverseRotationMatrix(
		FPlane(FMath::Cos(Rotation), -FMath::Sin(Rotation), 0, 0),
		FPlane(FMath::Sin(Rotation), FMath::Cos(Rotation), 0, 0),
		FPlane(0, 0, 1, 0),
		FPlane(0, 0, 0, 1));

	FMatrix CropingUVTransformationMatrix =
		TranslateMatrix *
		PostRotationMatrix *
		RotationMatrix *
		ScaleMatrix *
		PreRotationMatrix;

	if (OutCropingUVTransformationMatrix)
	{
		*OutCropingUVTransformationMatrix = CropingUVTransformationMatrix;
	}

	if (OutUncropingUVTransformationMatrix)
	{
		*OutUncropingUVTransformationMatrix =
			PostRotationMatrix *
			InverseRotationMatrix *
			InverseScaleMatrix *
			PreRotationMatrix;

		OutUncropingUVTransformationMatrix->M[3][0] = -(
			OutUncropingUVTransformationMatrix->M[0][0] * CropingUVTransformationMatrix.M[3][0] +
			OutUncropingUVTransformationMatrix->M[1][0] * CropingUVTransformationMatrix.M[3][1]);
		OutUncropingUVTransformationMatrix->M[3][1] = -(
			OutUncropingUVTransformationMatrix->M[0][1] * CropingUVTransformationMatrix.M[3][0] +
			OutUncropingUVTransformationMatrix->M[1][1] * CropingUVTransformationMatrix.M[3][1]);
	}
}
