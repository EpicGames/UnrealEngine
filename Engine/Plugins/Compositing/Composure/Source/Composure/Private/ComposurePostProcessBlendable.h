// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlendableInterface.h"
#include "ComposurePostProcessBlendable.generated.h"


/**
 * Private blendable interface for  UComposurePostProcessPass.
 */
UCLASS(NotBlueprintType)
class UComposurePostProcessBlendable
	: public UObject
	, public IBlendableInterface
{
	GENERATED_UCLASS_BODY()

public:

	// Begins IBlendableInterface
	void OverrideBlendableSettings(class FSceneView& View, float Weight) const override;
	// Ends IBlendableInterface

private:
	// Current player camera manager the target is bind on.
	UPROPERTY(Transient)
	TObjectPtr<class UComposurePostProcessPass> Target;
	
	friend class UComposurePostProcessPass;
};
