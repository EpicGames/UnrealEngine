// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposurePostProcessPass.h"
#include "ComposurePostProcessBlendable.h"

#include "Components/SceneCaptureComponent2D.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "SceneView.h"
#include "ComposureInternals.h"
#include "UObject/ConstructorHelpers.h"


UComposurePostProcessBlendable::UComposurePostProcessBlendable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Target(nullptr)
{}

void UComposurePostProcessBlendable::OverrideBlendableSettings(class FSceneView& View, float Weight) const
{
	check(Target);
	Target->OverrideBlendableSettings(View, Weight);
}


UComposurePostProcessPass::UComposurePostProcessPass( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
	, SceneCapture(nullptr)
	, BlendableInterface(nullptr)
{
	bWantsInitializeComponent = true;

	// Sets a default setup material that sets black.
	COMPOSURE_GET_MATERIAL(Material, SetupMaterial, "PassSetup/", "ComposureBeforeTranslucencySetBlack");
	SetSetupMaterial(SetupMaterial);

	bAutoActivate = true;
}

void UComposurePostProcessPass::SetSetupMaterial(UMaterialInterface* Material)
{
	if (!Material)
	{
		UE_LOGF(Composure, Error, "Can't set setup a null material to a UComposurePostProcessPass.");
		return;
	}
	UMaterial* BaseMaterial = Material->GetMaterial();
	if (BaseMaterial->MaterialDomain != MD_PostProcess)
	{
		UE_LOGF(Composure, Error,
			"Can't set setup material %ls: is not in the post process domain.",
			*BaseMaterial->GetName());
		return;
	}
	else if (Material->GetBlendableLocation(BaseMaterial) != BL_SceneColorBeforeDOF)
	{
		UE_LOGF(Composure, Error,
			"Can't set setup material %ls: is not at the before translucency post process location.",
			*BaseMaterial->GetName());
		return;
	}

	SetupMaterial = Material;
}

UTextureRenderTarget2D* UComposurePostProcessPass::GetOutputRenderTarget() const
{
	return SceneCapture->TextureTarget;
}

void UComposurePostProcessPass::SetOutputRenderTarget(UTextureRenderTarget2D* RenderTarget)
{
	if (SceneCapture)
	{
		SceneCapture->TextureTarget = RenderTarget;
	}
}

void UComposurePostProcessPass::Activate(bool bReset)
{
	Super::Activate(bReset);

	if (IsActive())
	{
		if (SceneCapture == nullptr)
		{
			SceneCapture = NewObject<USceneCaptureComponent2D>(this, TEXT("SceneCapture"));
			// Create underlying scene capture.
			SceneCapture->AttachToComponent(this, FAttachmentTransformRules::KeepRelativeTransform);
			// Avoid drawing any primitive by using the empty show only lists.
			SceneCapture->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
			// Avoid capturing every frame and on movement.
			SceneCapture->bCaptureEveryFrame = false;
			SceneCapture->bCaptureOnMovement = false;
			SceneCapture->bUseRayTracingIfEnabled = true;
			// Sets the capture source to final color to enable post processing.
			SceneCapture->CaptureSource = SCS_FinalColorLDR;

			SceneCapture->RegisterComponent();
		}

		if (BlendableInterface == nullptr)
		{
			// Create private blendable interface.
			BlendableInterface = NewObject<UComposurePostProcessBlendable>(this, TEXT("PostProcessBlendable"));
			BlendableInterface->Target = this;
		}
	}
}

void UComposurePostProcessPass::Deactivate()
{
	Super::Deactivate();

	if (BlendableInterface)
	{
		BlendableInterface->Target = nullptr;
		BlendableInterface = nullptr;
	}

	if (SceneCapture)
	{
		SceneCapture->DestroyComponent();
		SceneCapture = nullptr;
	}
}

void UComposurePostProcessPass::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	if (BlendableInterface)
	{
		BlendableInterface->Target = nullptr;
		BlendableInterface = nullptr;
	}

	if (SceneCapture)
	{
		SceneCapture->DestroyComponent();
		SceneCapture = nullptr;
	}

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void UComposurePostProcessPass::OverrideBlendableSettings(class FSceneView& View, float Weight) const
{
	// Clear any blendables that could have been set by post process volumes.
	View.FinalPostProcessSettings.BlendableManager = FBlendableManager();

	// Setup the post process materials.
	if (ensure(SetupMaterial))
	{
		SetupMaterial->OverrideBlendableSettings(View, Weight);
	}
	
	if (TonemapperReplacement)
	{
		TonemapperReplacement->OverrideBlendableSettings(View, Weight);
	}
}
