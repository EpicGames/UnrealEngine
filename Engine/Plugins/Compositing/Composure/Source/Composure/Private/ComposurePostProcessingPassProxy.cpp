// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposurePostProcessingPassProxy.h"
#include "ComposurePostProcessBlendable.h"
#include "Components/SceneCaptureComponent2D.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ComposureInternals.h" // for COMPOSURE_GET_MATERIAL()
#include "ComposureUtils.h" // for SetEngineShowFlagsForPostprocessingOnly()
#include "UObject/ConstructorHelpers.h"

/* UComposurePostProcessingPassProxy
 *****************************************************************************/

void UComposurePostProcessPassPolicy::SetupPostProcess_Implementation(USceneCaptureComponent2D* /*SceneCapture*/, UMaterialInterface*& OutTonemapperOverride)
{
	ensureMsgf(false, TEXT("'%s' is not properly overriding/implementing SetupPostProcess()."), *GetClass()->GetName());
	OutTonemapperOverride = nullptr;
}

/* UComposurePostProcessingPassProxy
 *****************************************************************************/

UComposurePostProcessingPassProxy::UComposurePostProcessingPassProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	COMPOSURE_CREATE_DYMAMIC_MATERIAL(Material, SetupMID, "PassSetup/", "ComposureSimpleSetupMaterial");
}

void UComposurePostProcessingPassProxy::PostInitProperties()
{
	Super::PostInitProperties();
	SetupMaterial = SetupMID;
}

void UComposurePostProcessingPassProxy::PostLoad()
{
	Super::PostLoad();

	// Having to account with sub-obj instancing overwriting this Property - alternative is 
	// to flag SetupMaterial as 'SkipSerialization', but that's pre-existing code we don't want to destabilize
	SetupMaterial = SetupMID;
}

void UComposurePostProcessingPassProxy::Execute(UTexture* PrePassInput, UComposurePostProcessPassPolicy* PostProcessPass)
{
	if (SetupMaterial)
	{
		if (!SetupMID || SetupMID->GetBaseMaterial() != SetupMaterial->GetBaseMaterial())
		{
			SetupMID = UMaterialInstanceDynamic::Create(SetupMaterial, this);
		}
		SetupMID->SetTextureParameterValue(TEXT("Input"), PrePassInput);
	}
	else
	{
		SetupMID = nullptr;
	}
	TGuardValue<decltype(SetupMaterial)> SetupMatGuard(SetupMaterial, SetupMID);
	
	if (PostProcessPass && SceneCapture)
	{
		// Disable as much stuff as possible using showflags. 
		FComposureUtils::SetEngineShowFlagsForPostprocessingOnly(SceneCapture->ShowFlags);

		UMaterialInterface* TonemapperOverride = nullptr;
		PostProcessPass->SetupPostProcess(SceneCapture, TonemapperOverride);

		bool bUseTonemapperOverride = true;
		if (TonemapperOverride)
		{
			UMaterial* OverrideMat = TonemapperOverride->GetBaseMaterial();
			if (OverrideMat->MaterialDomain != MD_PostProcess || OverrideMat->BlendableLocation != BL_ReplacingTonemapper)
			{
				bUseTonemapperOverride = false;
				UE_CLOGF(OverrideMat != nullptr, Composure, Warning, "Invalid tonemapper override supplied from: '%ls'", *PostProcessPass->GetClass()->GetName());
			}
		}
		TGuardValue<decltype(TonemapperReplacement)> TonemapperReplacementGuard(TonemapperReplacement, bUseTonemapperOverride ? TonemapperOverride : nullptr);

		// Ensure the scene cap isn't rendering any scene objects
		SceneCapture->ClearShowOnlyComponents();
		SceneCapture->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;

		// Adds the blendable to have programmatic control of FSceneView::FinalPostProcessSettings
		// in  UComposurePostProcessPass::OverrideBlendableSettings().
		SceneCapture->PostProcessSettings.AddBlendable(BlendableInterface, 1.0);

		SceneCapture->ProfilingEventName = PostProcessPass->GetClass()->GetName();

		// OverrideBlendableSettings() will do nothing (see UMaterialInterface::OverrideBlendableSettings) 
		// with these materials unless there is a ViewState from the capture component (see USceneCaptureComponent::GetViewState)
		TGuardValue<bool> ViewStateGuard(SceneCapture->bAlwaysPersistRenderingState, true);

		// Update the render target output.
		SceneCapture->CaptureScene();
	}	
}
