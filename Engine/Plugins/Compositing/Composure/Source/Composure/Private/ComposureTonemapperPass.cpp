// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureTonemapperPass.h"
#include "ComposurePostProcessBlendable.h"

#include "Components/SceneCaptureComponent2D.h"
#include "ComposureUtils.h"

/* FComposureTonemapperUtils
 *****************************************************************************/

void FComposureTonemapperUtils::ApplyTonemapperSettings(const FColorGradingSettings& ColorGradingSettings, const FFilmStockSettings& FilmStockSettings, const float ChromaticAberration, FPostProcessSettings& OutSettings)
{
	// Exports the settings to the scene capture's post process settings.
	ColorGradingSettings.ExportToPostProcessSettings(&OutSettings);
	FilmStockSettings.ExportToPostProcessSettings(&OutSettings);

	// Override some tone mapper non exposed settings to not have post process material changing them.
	{
		OutSettings.bOverride_SceneColorTint = true;
		OutSettings.SceneColorTint = FLinearColor::White;

		OutSettings.bOverride_VignetteIntensity = true;
		OutSettings.VignetteIntensity = 0;

		OutSettings.bOverride_FilmGrainIntensity = true;
		OutSettings.FilmGrainIntensity = 0;

		OutSettings.bOverride_BloomDirtMask = true;
		OutSettings.BloomDirtMask = nullptr;
		OutSettings.bOverride_BloomDirtMaskIntensity = true;
		OutSettings.BloomDirtMaskIntensity = 0;
	}

	OutSettings.bOverride_SceneFringeIntensity = true;
	OutSettings.SceneFringeIntensity = ChromaticAberration;
}

/* UComposureTonemapperPass
 *****************************************************************************/

UComposureTonemapperPass::UComposureTonemapperPass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Do not replace the engine's tonemapper.
	TonemapperReplacement = nullptr;

	ChromaticAberration = 0.0;
}

void UComposureTonemapperPass::TonemapToRenderTarget()
{
	// Disables as much stuf as possible using showflags. 
	FComposureUtils::SetEngineShowFlagsForPostprocessingOnly(SceneCapture->ShowFlags);

	FComposureTonemapperUtils::ApplyTonemapperSettings(ColorGradingSettings, FilmStockSettings, ChromaticAberration, SceneCapture->PostProcessSettings);
	//SceneCapture->ShowFlags.EyeAdaptation = true;

	// Adds the blendable to have programmatic control of FSceneView::FinalPostProcessSettings
	// in  UComposurePostProcessPass::OverrideBlendableSettings().
	SceneCapture->PostProcessSettings.AddBlendable(BlendableInterface, 1.0);

	SceneCapture->ProfilingEventName = TEXT("ComposureTonemapperPass");

	// OverrideBlendableSettings() will do nothing (see UMaterialInterface::OverrideBlendableSettings) 
	// with these materials unless there is a ViewState from the capture component (see USceneCaptureComponent::GetViewState)
	TGuardValue<bool> ViewStateGuard(SceneCapture->bAlwaysPersistRenderingState, true);

	// Update the render target output.
	SceneCapture->CaptureScene();
}

/* UComposureTonemapperPassPolicy
 *****************************************************************************/

void UComposureTonemapperPassPolicy::SetupPostProcess_Implementation(USceneCaptureComponent2D* SceneCapture, UMaterialInterface*& OutTonemapperOverride)
{
	// Do not replace the engine's tonemapper.
	OutTonemapperOverride = nullptr;

	FComposureTonemapperUtils::ApplyTonemapperSettings(ColorGradingSettings, FilmStockSettings, ChromaticAberration, SceneCapture->PostProcessSettings);
	//SceneCapture->ShowFlags.EyeAdaptation = true;
}
