// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureUVMap.h"

#include "Engine/Texture.h"
#include "Materials/MaterialInstanceDynamic.h"


static inline void SetUVMatrix(
	class UMaterialInstanceDynamic* MID,
	const FName& Matrix2x2ParameterName,
	const FName& Translate2x2ParameterName,
	const FMatrix& UVMatrix)
{
	FLinearColor Matrix2x2;
	FLinearColor Translate;

	Matrix2x2.R = UVMatrix.M[0][0];
	Matrix2x2.G = UVMatrix.M[0][1];
	Matrix2x2.B = UVMatrix.M[1][0];
	Matrix2x2.A = UVMatrix.M[1][1];

	Translate.R = UVMatrix.M[3][0];
	Translate.G = UVMatrix.M[3][1];
	Translate.B = 0.0;
	Translate.A = 0.0;

	MID->SetVectorParameterValue(Matrix2x2ParameterName, Matrix2x2);
	MID->SetVectorParameterValue(Translate2x2ParameterName, Translate);
}


void FComposureUVMapSettings::SetMaterialParameters(class UMaterialInstanceDynamic* MID) const
{
	SetUVMatrix(MID, TEXT("PreUVMapMatrix"), TEXT("PreUVMapTranslate"), PreUVDisplacementMatrix);
	SetUVMatrix(MID, TEXT("PostUVMapMatrix"), TEXT("PostUVMapTranslate"), PostUVDisplacementMatrix);
	MID->SetVectorParameterValue(TEXT("UVMapTextureDecoding"), 
		FLinearColor(DisplacementDecodeParameters.X, DisplacementDecodeParameters.Y, 0, 0));
	MID->SetTextureParameterValue(TEXT("UVDisplacementMapTexture"), DisplacementTexture);
	MID->SetScalarParameterValue(TEXT("bUseBlueAndAlphaChannels"), bUseDisplacementBlueAndAlphaChannels ? 1.f : 0.f);
}
