// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureUtils.h"

#include "ShowFlags.h"


// static
void FComposureUtils::SetEngineShowFlagsForPostprocessingOnly(FEngineShowFlags& EngineShowFlags)
{
	EngineShowFlags.DynamicShadows = false;
	EngineShowFlags.ReflectionEnvironment = false;
	EngineShowFlags.ScreenSpaceReflections = false;
	EngineShowFlags.ScreenSpaceAO = false;
	EngineShowFlags.LightShafts = false;
	EngineShowFlags.Lighting = false;
	EngineShowFlags.SkyLighting = false;
	EngineShowFlags.DeferredLighting = false;
	EngineShowFlags.Decals = false;
	EngineShowFlags.Translucency = false;
	EngineShowFlags.AntiAliasing = false;
	EngineShowFlags.TemporalAA = false;
	EngineShowFlags.MotionBlur = false;
	EngineShowFlags.Bloom = false;
	EngineShowFlags.EyeAdaptation = false;
	EngineShowFlags.VolumetricFog = false;
	EngineShowFlags.Atmosphere = false;
	EngineShowFlags.Fog = false;
	EngineShowFlags.LocalExposure = false;
	EngineShowFlags.GlobalIllumination = false;
	EngineShowFlags.LumenReflections = false;

#if !UE_BUILD_OPTIMIZED_SHOWFLAGS
	// Development-only flags
	EngineShowFlags.ReflectionOverride = false;
	EngineShowFlags.DepthOfField = false;
#endif
}

// static
FVector2D FComposureUtils::GetRedGreenUVFactorsFromChromaticAberration(float ChromaticAberrationAmount)
{
	// Wavelength of primaries in nm
	const float PrimaryR = 611.3f;
	const float PrimaryG = 549.1f;
	const float PrimaryB = 464.3f;

	// Simple lens chromatic aberration is roughly linear in wavelength
	float ScaleR = 0.007f * (PrimaryR - PrimaryB);
	float ScaleG = 0.007f * (PrimaryG - PrimaryB);
	return FVector2D(1.0f / (1.0f + ChromaticAberrationAmount * ScaleR), 1.0f / (1.0f + ChromaticAberrationAmount * ScaleG));
}
