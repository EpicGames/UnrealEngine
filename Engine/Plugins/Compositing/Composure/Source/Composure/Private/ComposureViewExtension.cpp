// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureViewExtension.h"
#include "ComposurePipelineBaseActor.h"
#include "SceneView.h"

//------------------------------------------------------------------------------
FComposureViewExtension::FComposureViewExtension(const FAutoRegister& AutoRegister, AComposurePipelineBaseActor* Owner)
	: FSceneViewExtensionBase(AutoRegister)
	, AssociatedPipelineObj(Owner)
{}

//------------------------------------------------------------------------------
void FComposureViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	if (AssociatedPipelineObj.IsValid())
	{
		bool bHasSceneCaptureView = false;
		bool bCameraCutThisFrame = false;
		for (const FSceneView* SceneView : InViewFamily.Views)
		{
			if (SceneView)
			{
				if (SceneView->bCameraCut)
				{
					bCameraCutThisFrame = true;
				}

				if (SceneView->bIsSceneCapture)
				{
					bHasSceneCaptureView = true;
				}
			}
		}
		

		// Note: Previously the engine wasn't calling BeginRenderViewFamily on scene captures.
		// Now that it does, we explicitely disable this case to prevent recursive scene capture rendering.
		if (!bHasSceneCaptureView)
		{
			AComposurePipelineBaseActor* Owner = AssociatedPipelineObj.Get();
			Owner->EnqueueRendering(bCameraCutThisFrame);
		}
	}
}

//------------------------------------------------------------------------------
int32 FComposureViewExtension::GetPriority() const
{
	if (AssociatedPipelineObj.IsValid())
	{
		return AssociatedPipelineObj->GetRenderPriority();
	}
	return FSceneViewExtensionBase::GetPriority();
}

//------------------------------------------------------------------------------
bool FComposureViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext&) const
{
	bool bActive = false;
	if (AssociatedPipelineObj.IsValid())
	{
		AComposurePipelineBaseActor* Owner = AssociatedPipelineObj.Get();
		bActive = Owner->IsActivelyRunning();
	}
	return bActive;
}

