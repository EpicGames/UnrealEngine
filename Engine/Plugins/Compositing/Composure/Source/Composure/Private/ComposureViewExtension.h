// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "SceneViewExtension.h"

class AComposurePipelineBaseActor;

/**
 *	
 */
class FComposureViewExtension : public FSceneViewExtensionBase
{
public:
	FComposureViewExtension(const FAutoRegister& AutoRegister, AComposurePipelineBaseActor* Owner);

public:
	//~ ISceneViewExtension interface
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual int32 GetPriority() const override;
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;

private:
	TWeakObjectPtr<AComposurePipelineBaseActor> AssociatedPipelineObj;
};
