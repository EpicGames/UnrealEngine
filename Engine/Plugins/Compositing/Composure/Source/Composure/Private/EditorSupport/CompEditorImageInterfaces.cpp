// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSupport/CompImageColorPickerInterface.h"

UCompEditorImagePreviewInterface::UCompEditorImagePreviewInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

UCompImageColorPickerInterface::UCompImageColorPickerInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}
