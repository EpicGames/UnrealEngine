// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSupport/CompFreezeFrameController.h"

namespace CompFreezeFrameController_Impl
{
	static int32 DummyMaskRef = 0x00;
}

FCompFreezeFrameController::FCompFreezeFrameController(EForceInit DefaultInit)
	: FreezeFlags(CompFreezeFrameController_Impl::DummyMaskRef)
{
	if (DefaultInit != ForceInitToZero)
	{
		Lock();
	}
}

bool FCompFreezeFrameController::SetFreezeFlags(ETargetUsageFlags InFreezeFlags, bool bClearOthers, const FFreezeFrameControlHandle& InLockKey)
{
	if (!IsLocked() || InLockKey == LockKey)
	{
		if (bClearOthers)
		{
			FreezeFlags  = (int32)InFreezeFlags;
		}
		else
		{
			FreezeFlags |= (int32)InFreezeFlags;
		}
		return true;
	}
	return false;
}

bool FCompFreezeFrameController::ClearFreezeFlags(ETargetUsageFlags InFreezeFlags, const FFreezeFrameControlHandle& InLockKey)
{
	if (!IsLocked() || InLockKey == LockKey)
	{
		FreezeFlags &= ~(int32)InFreezeFlags;
		return true;
	}
	return false;
}

bool FCompFreezeFrameController::ClearFreezeFlags(const FFreezeFrameControlHandle& InLockKey)
{
	return ClearFreezeFlags((ETargetUsageFlags)0xff, InLockKey);
}

bool FCompFreezeFrameController::HasAnyFlags(ETargetUsageFlags InFreezeFlags)
{
	return (FreezeFlags & (int32)InFreezeFlags) != 0;
}

bool FCompFreezeFrameController::HasAllFlags(ETargetUsageFlags InFreezeFlags)
{
	return (FreezeFlags & (int32)InFreezeFlags) == (int32)InFreezeFlags;
}
