// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSupport/CompositingEditorSupportLibrary.h"

/* UCompositingPickerAsyncTask
 *****************************************************************************/

#include "EditorSupport/ICompositingEditor.h"
#include "EditorSupport/WeakUInterfacePtr.h"

UCompositingPickerAsyncTask* UCompositingPickerAsyncTask::OpenCompositingPicker(UTextureRenderTarget2D* PickerTarget, UTexture* DisplayImage, FText WindowTitle, const bool bAverageColorOnDrag, const bool bUseImplicitGamma)
{
	UCompositingPickerAsyncTask* PickerTask = NewObject<UCompositingPickerAsyncTask>();
	PickerTask->bUseImplicitGamma = bUseImplicitGamma;

	PickerTask->Open(PickerTarget, DisplayImage, bAverageColorOnDrag, WindowTitle);

	return PickerTask;
}

#if WITH_EDITOR
UTexture* UCompositingPickerAsyncTask::GetEditorPreviewImage()
{
	return PickerDisplayImage;
}

UTexture* UCompositingPickerAsyncTask::GetColorPickerDisplayImage()
{
	return PickerDisplayImage;
}

UTextureRenderTarget2D* UCompositingPickerAsyncTask::GetColorPickerTarget()
{
	return PickerTarget;
}

FCompFreezeFrameController* UCompositingPickerAsyncTask::GetFreezeFrameController()
{
	return nullptr;
}
#endif // WITH_EDITOR

void UCompositingPickerAsyncTask::Open(UTextureRenderTarget2D* InPickerTarget, UTexture* InDisplayImage, const bool bAverageColorOnDrag, const FText& WindowTitle)
{
	PickerTarget = InPickerTarget;
	PickerDisplayImage = InDisplayImage;

	bool bOpenSuccess = false;
#if WITH_EDITOR
	if (ICompositingEditor* CompositingEditor = ICompositingEditor::Get())
	{
		TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerInterfaceObj = this;
		
		TSharedPtr<SWindow> NewWindow = CompositingEditor->RequestCompositingPickerWindow(PickerInterfaceObj, bAverageColorOnDrag,
			ICompositingEditor::FPickerResultHandler::CreateUObject(this, &UCompositingPickerAsyncTask::InternalOnPick),
			FSimpleDelegate::CreateUObject(this, &UCompositingPickerAsyncTask::InternalOnCancel), WindowTitle);

		bOpenSuccess = NewWindow.IsValid();
	}
#endif

	if (!bOpenSuccess)
	{
		InternalOnCancel();
	}
}

void UCompositingPickerAsyncTask::InternalOnPick(const FVector2D& PickedUV, const FLinearColor& PickedColor, bool bInteractive)
{
	FEditorScriptExecutionGuard EdScriptGuard;

	if (bInteractive)
	{
		OnPick.Broadcast(PickedUV, PickedColor);
	}
	else
	{
		OnAccept.Broadcast(PickedUV, PickedColor);
		SetReadyToDestroy();
	}
}

void UCompositingPickerAsyncTask::InternalOnCancel()
{
	FEditorScriptExecutionGuard EdScriptGuard;

	OnCancel.Broadcast(FVector2D(-1.f, -1.f), FLinearColor::Black);
	SetReadyToDestroy();
}
