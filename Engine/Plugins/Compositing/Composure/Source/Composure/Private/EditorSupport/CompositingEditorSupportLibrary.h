// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintAsyncActionBase.h"
#include "EditorSupport/CompImageColorPickerInterface.h"
#include "CompositingEditorSupportLibrary.generated.h"

/* UCompositingPickerAsyncTask
 *****************************************************************************/

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPixelPicked, const FVector2D&, PickedUV, const FLinearColor&, SampledColor);

UCLASS()
class UCompositingPickerAsyncTask : public UBlueprintAsyncActionBase, public ICompImageColorPickerInterface
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Composure|Editor", meta=(DevelopmentOnly, BlueprintInternalUseOnly = "true"))
	static UCompositingPickerAsyncTask* OpenCompositingPicker(UTextureRenderTarget2D* PickerTarget, UTexture* DisplayImage, FText WindowTitle, const bool bAverageColorOnDrag = true, const bool bUseImplicitGamma = true);

	//~ Begin ICompImageColorPickerInterface interface
#if WITH_EDITOR
	virtual UTexture* GetEditorPreviewImage() override;
	virtual UTexture* GetColorPickerDisplayImage() override;
	virtual UTextureRenderTarget2D* GetColorPickerTarget() override;
	virtual FCompFreezeFrameController* GetFreezeFrameController() override;
	virtual bool UseImplicitGammaForPreview() const override { return bUseImplicitGamma; }
#endif
	//~ End ICompImageColorPickerInterface interface

public:
	UPROPERTY(BlueprintAssignable)
	FOnPixelPicked OnPick;

	UPROPERTY(BlueprintAssignable)
	FOnPixelPicked OnCancel;

	UPROPERTY(BlueprintAssignable)
	FOnPixelPicked OnAccept;

private:
	void Open(UTextureRenderTarget2D* PickerTarget, UTexture* DisplayImage, const bool bAverageColorOnDrag, const FText& WindowTitle);

	void InternalOnPick(const FVector2D& PickedUV, const FLinearColor& PickedColor, bool bInteractive);
	void InternalOnCancel();

	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> PickerTarget;
	UPROPERTY(Transient)
	TObjectPtr<UTexture> PickerDisplayImage;

	bool bUseImplicitGamma = true;
};
