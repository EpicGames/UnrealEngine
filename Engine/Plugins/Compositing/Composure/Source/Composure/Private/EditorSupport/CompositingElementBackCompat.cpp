// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElement.h"

namespace CompositingElement_Impl
{
}

void ACompositingElement::PostSerializeCompatUpgrade(const int32 ComposureVersion)
{
}

void ACompositingElement::PostLoadCompatUpgrade(const int32 ComposureVersion)
{
}
