// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingElement.h"
#include "CompositingElements/CompElementRenderTargetPool.h"
#include "CompositingElements/CompositingElementPasses.h"
#include "ComposurePlayerCompositingTarget.h"
#include "CompositingElements/CompositingElementPassUtils.h"
#include "EditorSupport/ICompositingEditor.h"
#include "Engine/Blueprint.h"


namespace CompositingElementEditorSupport_Impl
{
	template<typename T>
	T FindReplacedPass(const TArray<T>& PublicList, const TArray<T>& InternalList, const int32 ReplacedIndex);
}

template<typename T>
T CompositingElementEditorSupport_Impl::FindReplacedPass(const TArray<T>& PublicList, const TArray<T>& InternalList, const int32 ReplacedIndex)
{
	T FoundPass = nullptr;

	if (PublicList.IsValidIndex(ReplacedIndex))
	{
		if (T AlteredPass = PublicList[ReplacedIndex])
		{
			if (!InternalList.Contains(AlteredPass))
			{
				for (int32 PublicPassIndex = 0, InternalPassIndex = 0; PublicPassIndex <= ReplacedIndex && InternalPassIndex < InternalList.Num(); ++InternalPassIndex, ++PublicPassIndex)
				{
					while (PublicList[PublicPassIndex] == nullptr)
					{
						++PublicPassIndex;
					}

					if (PublicList[PublicPassIndex] != InternalList[InternalPassIndex])
					{
						FoundPass = InternalList[InternalPassIndex];
					}
				}
			}
		}
	}

	return FoundPass;
}

void ACompositingElement::SetEditorColorPickingTarget(UTextureRenderTarget2D* PickingTarget)
{
#if WITH_EDITOR
	ColorPickerTarget = PickingTarget;
#endif
}

void ACompositingElement::SetEditorColorPickerDisplayImage(UTexture* PickerDisplayImage)
{
#if WITH_EDITOR
	ColorPickerDisplayImage = PickerDisplayImage;
#endif
}

#if WITH_EDITOR
void ACompositingElement::OnBeginPreview()
{
	++PreviewCount;
}

UTexture* ACompositingElement::GetEditorPreviewImage()
{
	UTexture* PreviewImage = EditorPreviewImage;
	if (EditorPreviewImage == nullptr || bUsingDebugDisplayImage)
	{
		PreviewImage = CompositingTarget->GetDisplayTexture();
	}

	UClass* MyClass = GetClass();
	if (MyClass && MyClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(MyClass->ClassGeneratedBy))
		{
			if ((Blueprint->Status == EBlueprintStatus::BS_Error || Blueprint->Status == EBlueprintStatus::BS_Unknown))
			{
				PreviewImage = CompilerErrImage;
			}
		}
	}

	return PreviewImage;
}

void ACompositingElement::OnEndPreview()
{
	--PreviewCount;
}

bool ACompositingElement::UseImplicitGammaForPreview() const 
{
	UCompositingElementTransform* PreviewPass = GetPreviewPass();
	return (PreviewPass == nullptr) || !PreviewPass->IsPassEnabled();
}

UTexture* ACompositingElement::GetColorPickerDisplayImage()
{
	return (ColorPickerDisplayImage) ? ToRawPtr(ColorPickerDisplayImage) : (ColorPickerTarget ? ToRawPtr(ColorPickerTarget) : GetEditorPreviewImage());
}
UTextureRenderTarget2D* ACompositingElement::GetColorPickerTarget()
{
	return (ColorPickerTarget) ? ToRawPtr(ColorPickerTarget) : Cast<UTextureRenderTarget2D>(GetColorPickerDisplayImage());
}

FCompFreezeFrameController* ACompositingElement::GetFreezeFrameController()
{
	return &FreezeFrameController;
}

void ACompositingElement::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	const FName PropertyName = PropertyChangedChainEvent.PropertyChain.GetActiveNode()->GetValue()->GetFName();
	
	if (PropertyName == TEXT("ActorLabel"))//GET_MEMBER_NAME_CHECKED(ACompositingElement, ActorLabel))
	{
		CompShotIdName = *GetActorLabel();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACompositingElement, bUseSharedTargetPool))
	{
		if (RenderTargetPool.IsValid())
		{
			RenderTargetPool->ReleaseAssignedTargets(this);
			RenderTargetPool.Reset();
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACompositingElement, bAutoRun) || 
	         PropertyName == GET_MEMBER_NAME_CHECKED(ACompositingElement, bRunInEditor))
	{
		if (!IsActivelyRunning())
		{
			OnDisabled();
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ACompositingElement, bAutoRunChildElementsAndSelf))
	{
		SetAutoRunChildrenAndSelf(bAutoRunChildElementsAndSelf);
	}
	else if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		const bool bEditedInputsArray = PropertyName == GET_MEMBER_NAME_CHECKED(ACompositingElement, Inputs);
		const bool bEditedTransformPassesArray = PropertyName == GET_MEMBER_NAME_CHECKED(ACompositingElement, TransformPasses);
		const bool bEditedOutputsArray = PropertyName == GET_MEMBER_NAME_CHECKED(ACompositingElement, Outputs);
		if ((bEditedInputsArray || bEditedTransformPassesArray || bEditedOutputsArray))
		{
			if (PropertyChangedChainEvent.ChangeType == EPropertyChangeType::ValueSet ||
				(PropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayAdd &&
				((bEditedInputsArray && DefaultInputType) || (bEditedTransformPassesArray && DefaultTransformType) || (bEditedOutputsArray && DefaultOutputType))))
			{
				// Verify that there isn't another container property node further down in the chain (so we don't respond to array change events from a different array!)
				bool bWasNestedContainerChanged = false;
				FEditPropertyChain::TDoubleLinkedListNode* Node = PropertyChangedChainEvent.PropertyChain.GetActiveNode()->GetNextNode();
				while (!bWasNestedContainerChanged && Node && Node->GetValue())
				{
					bWasNestedContainerChanged = Node->GetValue()->IsA<FArrayProperty>() || Node->GetValue()->IsA<FSetProperty>() || Node->GetValue()->IsA<FMapProperty>();
					Node = Node->GetNextNode();
				}

				if (!bWasNestedContainerChanged)
				{
					const int32 ArrayIndex = PropertyChangedChainEvent.GetArrayIndex(PropertyName.ToString());
					if (bEditedInputsArray && Inputs.IsValidIndex(ArrayIndex))
					{
						if (PropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayAdd)
						{
							Inputs[ArrayIndex] = FCompositingElementPassUtils::NewInstancedSubObj<UCompositingElementInput>(/*Outer =*/this, DefaultInputType);
							Inputs[ArrayIndex]->PassName = MakeUniqueObjectName(this, UCompositingElementInput::StaticClass(), TEXT("InputPass"));
						}
						else if (Inputs[ArrayIndex] && Inputs[ArrayIndex]->PassName.IsNone())
						{
							if (UCompositingElementInput* ReplacedInput = CompositingElementEditorSupport_Impl::FindReplacedPass(Inputs, GetInternalInputsList(), ArrayIndex))
							{
								Inputs[ArrayIndex]->PassName = ReplacedInput->PassName;
							}
							else if (!GetInternalInputsList().Contains(Inputs[ArrayIndex]))
							{
								Inputs[ArrayIndex]->PassName = MakeUniqueObjectName(this, UCompositingElementInput::StaticClass(), TEXT("InputPass"));
							}
						}
						
						RefreshInternalInputsList();
					}
					else if (bEditedTransformPassesArray && TransformPasses.IsValidIndex(ArrayIndex))
					{
						if (PropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayAdd)
						{
							TransformPasses[ArrayIndex] = FCompositingElementPassUtils::NewInstancedSubObj<UCompositingElementTransform>(/*Outer =*/this, DefaultTransformType);
							TransformPasses[ArrayIndex]->PassName = MakeUniqueObjectName(this, UCompositingElementTransform::StaticClass(), TEXT("TransformPass"));
						}
						else if (TransformPasses[ArrayIndex] && TransformPasses[ArrayIndex]->PassName.IsNone())
						{
							if (UCompositingElementTransform* ReplacedInput = CompositingElementEditorSupport_Impl::FindReplacedPass(TransformPasses, GetInternalTransformsList(), ArrayIndex))
							{
								TransformPasses[ArrayIndex]->PassName = ReplacedInput->PassName;
							}
							else if (!GetInternalTransformsList().Contains(TransformPasses[ArrayIndex]))
							{
								TransformPasses[ArrayIndex]->PassName = MakeUniqueObjectName(this, UCompositingElementTransform::StaticClass(), TEXT("TransformPass"));
							}
						}
						
						RefreshInternalTransformsList();
					}
					else if (bEditedOutputsArray && Outputs.IsValidIndex(ArrayIndex))
					{
						if (PropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayAdd)
						{
							Outputs[ArrayIndex] = FCompositingElementPassUtils::NewInstancedSubObj<UCompositingElementOutput>(/*Outer =*/this, DefaultOutputType);
							Outputs[ArrayIndex]->PassName = MakeUniqueObjectName(this, UCompositingElementOutput::StaticClass(), TEXT("OutputPass"));
						}
						else if (Outputs[ArrayIndex] && Outputs[ArrayIndex]->PassName.IsNone())
						{
							if (UCompositingElementOutput* ReplacedInput = CompositingElementEditorSupport_Impl::FindReplacedPass(Outputs, GetInternalOutputsList(), ArrayIndex))
							{
								Outputs[ArrayIndex]->PassName = ReplacedInput->PassName;
							}
							else if (!GetInternalOutputsList().Contains(Outputs[ArrayIndex]))
							{
								Outputs[ArrayIndex]->PassName = MakeUniqueObjectName(this, UCompositingElementOutput::StaticClass(), TEXT("OutputPass"));
							}
						}
						
						RefreshInternalOutputsList();
					}
				}
			}
		}
	}

	if (ICompositingEditor* CompositingEditor = ICompositingEditor::Get())
	{
		CompositingEditor->RequestRedraw();
	}
	
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);
}

void ACompositingElement::PostEditUndo()
{
	Super::PostEditUndo();

	if (!IsActivelyRunning())
	{
		SetDebugDisplayImage(DisabledMsgImage);
	}

	RefreshAllInternalPassLists();
}

void ACompositingElement::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (Parent)
	{
		Parent->AttachAsChildLayer(this);
	}

	RefreshAllInternalPassLists();
}

void ACompositingElement::OnConstruction(const FTransform& Transform)
{
	OnConstructed.Broadcast(this);
	Super::OnConstruction(Transform);
}

UCompositingElementTransform* ACompositingElement::GetPreviewPass() const
{
	if (Parent && PreviewTransformSource == EInheritedSourceType::Inherited)
	{
		return Parent->GetPreviewPass();
	}
	return PreviewTransform;
}

bool ACompositingElement::IsPreviewing() const
{
	ensure(PreviewCount >= 0);
	return PreviewCount > 0 || CompositingTarget->IsPreviewing();
}

void ACompositingElement::OnPIEStarted(bool /*bIsSimulating*/)
{
	if (IsAutoRunSuspended())
	{
		if (RenderTargetPool.IsValid())
		{
			RenderTargetPool->ReleaseAssignedTargets(this);
		}
		SetDebugDisplayImage(SuspendedDbgImage);
	}
}

void ACompositingElement::SetDebugDisplayImage(UTexture* DebugDisplayImg)
{
	bUsingDebugDisplayImage = (DebugDisplayImg != nullptr);
	if (bUsingDebugDisplayImage)
	{
		PassResultsTable.SetMostRecentResult(nullptr);

		if (CompositingTarget)
		{
			CompositingTarget->SetDisplayTexture(DebugDisplayImg);
			CompositingTarget->SetUseImplicitGammaForPreview(true);
		}
	}
}
#endif // WITH_EDITOR
