// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorSupport/ICompositingEditor.h"
#include "Features/IModularFeatures.h"

ICompositingEditor* ICompositingEditor::Get()
{
	IModularFeatures& ModularFeatureManager = IModularFeatures::Get();
	if (ModularFeatureManager.GetModularFeatureImplementationCount(ICompositingEditor::GetModularFeatureName()) > 0)
	{
		return &ModularFeatureManager.GetModularFeature<ICompositingEditor>(ICompositingEditor::GetModularFeatureName());
	}
	return nullptr;
}
