// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneComposureExportSectionTemplate.h"
#include "Engine/Texture.h"
#include "MovieSceneCaptureEnvironment.h"
#include "Evaluation/MovieSceneExecutionTokens.h"
#include "MovieScene/IMovieSceneComposureExportClient.h"

#include "Evaluation/PreAnimatedState/MovieSceneRestoreStateParams.h"
#include "ImageWriteStream.h"
#include "ImagePixelData.h"
#include "SceneView.h"
#include "Protocols/UserDefinedCaptureProtocol.h"
#include "SceneViewExtension.h"
#include "CompositingElement.h"
#include "Components/SceneCaptureComponent2D.h"
#include "BufferVisualizationData.h"

#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "Async/Async.h"


// Shared persistent data key that pertains to all capture tracks for a sequence
static FSharedPersistentDataKey ComposureExportSharedKey(FMovieSceneSharedDataId::Allocate(), FMovieSceneEvaluationOperand());

/** Iterator used to gather valid visualization targets */
struct FBufferVisualizationIterator
{
	/** Array view of the desired buffer names to capture */
	TArrayView<const FString> BuffersToCapture;
	/** Reference to the post processing settings to add vialization materials to */
	FFinalPostProcessSettings& FinalPostProcessSettings;

	void ProcessValue(const FString& InName, UMaterialInterface* Material, const FText& InText)
	{
		if (BuffersToCapture.Contains(InName) || BuffersToCapture.Contains(InText.ToString()))
		{
			FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Add(Material);
		}
	}
};

/** Scene view extension that is added to scene captures when they want to capture intermediate buffers from the composition graph */
struct FExportIntermediateBuffersViewExtension : ISceneViewExtension
{
	/**
	 * Incremented when a frame is to be exported, decremented when the extension has been used for a frame
	 * Implemented this way to alleviate order dependency problems between threads, and IsCapturing()
	 */
	int32 NumOutstandingFrames;

	/**
	 * Create a new scene view extension with the specified parameters
	 */
	static TSharedRef<FExportIntermediateBuffersViewExtension, ESPMode::ThreadSafe> Create(ACompositingElement* InCompShotElement, USceneCaptureComponent2D* InSceneCapture, const TArray<FString>& InBuffersToCapture);

	virtual ~FExportIntermediateBuffersViewExtension() = default;

private:

	virtual void SetupViewFamily( FSceneViewFamily& InViewFamily ) override;
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;

	/** Makes a new endpoint for an image pixel pipe that just forwards the pixels onto the capture protocol */
	static TFunction<void(TUniquePtr<FImagePixelData>&&)> MakeForwardingEndpoint(const FCapturedPixelsID& BufferID, const FFrameMetrics& CurrentFrameMetrics);

	/** The base name to use for the {element} part of the pixels' ID */
	FName BaseBufferName;
	/** Array of user-provided buffers that want to be exported */
	TArray<FString> BuffersToCapture;
};


/**
 * Structure used as the authoritative representation of which passes to export for a given ACompositingElement
 * This is the templated type used for the blending accumulator/actuator
 */
struct FMovieSceneComposureExportPasses
{
	/** Map of internal transform pass name (or NAME_None for the Output), to the export config options */
	TSortedMap<FName, FMovieSceneComposureExportPass, FDefaultAllocator, FNameFastLess> PassesToExport;

	void AddPass(const FMovieSceneComposureExportPass& InPass, ACompositingElement* CompShotElement)
	{
		// If one exists already with this name, we error if the properties differ at all
		const FMovieSceneComposureExportPass* ExistingOptions = PassesToExport.Find(InPass.TransformPassName);

		if (ExistingOptions)
		{
			if (InPass.bRenamePass != ExistingOptions->bRenamePass || (InPass.bRenamePass == true && InPass.ExportedAs != ExistingOptions->ExportedAs) )
			{
				UE_LOGF(LogMovieScene, Error, "Encountered conflicting entries for exporting composure pass %ls from element %ls.", *InPass.TransformPassName.ToString(), *CompShotElement->GetName());
			}
		}
		else
		{
			PassesToExport.Add(InPass.TransformPassName, InPass);
		}
	}
};


/**
 * Manager class that persists for the entire duration of a sequence's evaluation that handles exporting each pass of each element in the sequence
 */
struct FComposureShotElementCaptureManager : IPersistentEvaluationData
{
	typedef TArray<TSharedPtr<FExportIntermediateBuffersViewExtension, ESPMode::ThreadSafe>> FExtensionArray;

	FComposureShotElementCaptureManager();
	~FComposureShotElementCaptureManager();

	/**
	 * Called to capture the specified pass from a composure element.
	 */
	void CaptureShotElementPass(ACompositingElement* CompShotElement, const FMovieSceneComposureExportPass& InPass);

	/**
	 * Called when a section stops evaluating, to prevent it from being exported any more
	 */
	void StopCapturingShotElementPass(ACompositingElement* CompShotElement, FName PassName);

	/**
	 * Called at the end of the frame to reset the bound passes ready for the next frame
	 */
	void StartFrameReset();

private:

	/**
	 * Called whenever any pass from a composure element that we are exporting is rendered
	 */
	void HandleOnFinalPassRendered(ACompositingElement* CompShotElement, UTexture* Texture);
	void HandleOnTransformPassRendered(ACompositingElement* CompShotElement, UTexture* Texture, FName PassName);

	void ExportPass(ACompositingElement* CompShotElement, UTexture* Texture, FName PassName);

	struct FBoundPasses
	{
		FDelegateHandle OnFinalPassRenderedHandle;
		FDelegateHandle OnTransformPassRenderedHandle;
		FExtensionArray SceneCaptureExtensions;
		FMovieSceneComposureExportPasses Passes;
	};

	/** Comp element -> bound pass information */
	TMap<FObjectKey, FBoundPasses> CompElementToPasses;

	/** Transient initializer object that is used as an interop for blueprint functionality */
	TStrongObjectPtr<UMovieSceneComposureExportInitializer> ExportInitializerObject;
};


struct FStopCapturingShotElementPassToken : IMovieScenePreAnimatedToken
{
	FName PassName;
	FStopCapturingShotElementPassToken(FName InPassName)
		: PassName(InPassName)
	{}

	virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params)
	{
		IMovieScenePlayer* Player = Params.GetTerminalPlayer();
		if (!ensure(Player))
		{
			return;
		}

		FPersistentEvaluationData PersistentData(*Player);

		FComposureShotElementCaptureManager* CaptureManager = PersistentData.Find<FComposureShotElementCaptureManager>(ComposureExportSharedKey);
		if (CaptureManager)
		{
			CaptureManager->StopCapturingShotElementPass(CastChecked<ACompositingElement>(&Object), PassName);
		}
	}
};

struct FStopCapturingShotElementPassTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	FName PassName;
	FStopCapturingShotElementPassTokenProducer(FName InPassName)
		: PassName(InPassName)
	{}

	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override
	{
		return FStopCapturingShotElementPassToken(PassName);
	}
};

struct FCaptureShotElementPassToken : IMovieSceneExecutionToken
{
	FMovieSceneComposureExportPass CapturePass;

	FCaptureShotElementPassToken(const FMovieSceneComposureExportPass& InCapturePass)
		: CapturePass(InCapturePass)
	{}

	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		FComposureShotElementCaptureManager& CaptureManager = PersistentData.GetOrAdd<FComposureShotElementCaptureManager>(ComposureExportSharedKey);

		for (TWeakObjectPtr<> WeakObject : Player.FindBoundObjects(Operand))
		{
			ACompositingElement* CompShotElement = Cast<ACompositingElement>(WeakObject.Get());
			if (CompShotElement)
			{
				static TMovieSceneAnimTypeIDContainer<FName> AnimTypeIDs;

				Player.SavePreAnimatedState(*CompShotElement, AnimTypeIDs.GetAnimTypeID(CapturePass.TransformPassName), FStopCapturingShotElementPassTokenProducer(CapturePass.TransformPassName));
				CaptureManager.CaptureShotElementPass(CompShotElement, CapturePass);
			}
		}
	}
};

FMovieSceneComposureExportSectionTemplate::FMovieSceneComposureExportSectionTemplate(const UMovieSceneComposureExportTrack& Track)
	: Pass(Track.Pass)
{}

void FMovieSceneComposureExportSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	UUserDefinedCaptureProtocol* CaptureProtocol = Cast<UUserDefinedCaptureProtocol>(UMovieSceneCaptureEnvironment::FindImageCaptureProtocol());
	if (CaptureProtocol && CaptureProtocol->IsCapturing())
	{
		ExecutionTokens.Add(FCaptureShotElementPassToken(Pass));
	}
}


TArray<TSharedPtr<FExportIntermediateBuffersViewExtension, ESPMode::ThreadSafe>> UMovieSceneComposureExportInitializer::InitializeCompShotElement(ACompositingElement* CompShotElement)
{
	check(CompShotElement);
	TmpExtensions.Reset();

	if (CompShotElement->GetClass()->ImplementsInterface(UMovieSceneComposureExportClient::StaticClass()))
	{
		IMovieSceneComposureExportClient::Execute_InitializeForExport(CompShotElement, this);
	}

	return MoveTemp(TmpExtensions);
}

void UMovieSceneComposureExportInitializer::ExportSceneCaptureBuffers(ACompositingElement* CompShotElement, USceneCaptureComponent2D* SceneCapture, const TArray<FString>& BuffersToExport)
{
	TmpExtensions.Add(FExportIntermediateBuffersViewExtension::Create(CompShotElement, SceneCapture, BuffersToExport));
}

TSharedRef<FExportIntermediateBuffersViewExtension, ESPMode::ThreadSafe> FExportIntermediateBuffersViewExtension::Create(ACompositingElement* InCompShotElement, USceneCaptureComponent2D* InSceneCapture, const TArray<FString>& InBuffersToCapture)
{
	TSharedRef<FExportIntermediateBuffersViewExtension, ESPMode::ThreadSafe> NewExtension = MakeShareable(new FExportIntermediateBuffersViewExtension);
	check(InCompShotElement);

	NewExtension->NumOutstandingFrames = 0;
	NewExtension->BuffersToCapture     = InBuffersToCapture;
	NewExtension->BaseBufferName       = InCompShotElement->GetFName();

	InSceneCapture->SceneViewExtensions.Add(NewExtension);
	return NewExtension;
}

void FExportIntermediateBuffersViewExtension::SetupViewFamily( FSceneViewFamily& InViewFamily )
{
	if (NumOutstandingFrames > 0)
	{
		--NumOutstandingFrames;
	}
}

void FExportIntermediateBuffersViewExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	InView.FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Empty();
	InView.FinalPostProcessSettings.BufferVisualizationPipes.Empty();

	UUserDefinedCaptureProtocol* CaptureProtocol = Cast<UUserDefinedCaptureProtocol>(UMovieSceneCaptureEnvironment::FindImageCaptureProtocol());
	check(CaptureProtocol);

	FFrameMetrics CurrentFrameMetrics = CaptureProtocol->GetCurrentFrameMetrics();

	FBufferVisualizationIterator Iterator{BuffersToCapture, InView.FinalPostProcessSettings};
	GetBufferVisualizationData().IterateOverAvailableMaterials(Iterator);

	FCapturedPixelsID BufferID;
	BufferID.Identifiers.Add("Element", BaseBufferName);


	for (UMaterialInterface* VisMaterial : Iterator.FinalPostProcessSettings.BufferVisualizationOverviewMaterials)
	{
		BufferID.Identifiers.Add("Pass", VisMaterial->GetFName());

		auto BufferPipe = MakeShared<FImagePixelPipe, ESPMode::ThreadSafe>();
		BufferPipe->AddEndpoint(MakeForwardingEndpoint(BufferID, CurrentFrameMetrics));

		InView.FinalPostProcessSettings.BufferVisualizationPipes.Add(VisMaterial->GetFName(), BufferPipe);
	}


	int32 NumValidMaterials = InView.FinalPostProcessSettings.BufferVisualizationPipes.Num();
	if (NumValidMaterials > 0)
	{
		CaptureProtocol->ReportOutstandingWork(NumValidMaterials);
		InView.FinalPostProcessSettings.bBufferVisualizationDumpRequired = true;
	}
}

bool FExportIntermediateBuffersViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext&) const
{
	return NumOutstandingFrames > 0;
}

TFunction<void(TUniquePtr<FImagePixelData>&&)> FExportIntermediateBuffersViewExtension::MakeForwardingEndpoint(const FCapturedPixelsID& BufferID, const FFrameMetrics& CurrentFrameMetrics)
{
	auto OnImageReceived = [BufferID, CurrentFrameMetrics](TUniquePtr<FImagePixelData>&& InOwnedImage)
	{
		TSharedPtr<FImagePixelData, ESPMode::ThreadSafe> SharedPixels = MakeShareable(InOwnedImage.Release());

		AsyncTask(ENamedThreads::GameThread, [SharedPixels, BufferID, CurrentFrameMetrics]
			{
				UUserDefinedCaptureProtocol* CaptureProtocol = Cast<UUserDefinedCaptureProtocol>(UMovieSceneCaptureEnvironment::FindImageCaptureProtocol());
				if (CaptureProtocol)
				{
					FCapturedPixels CapturedPixels{SharedPixels};
					CaptureProtocol->OnPixelsReceivedImpl(CapturedPixels, BufferID, CurrentFrameMetrics);
				}
			}
		);
	};

	return OnImageReceived;
}


FComposureShotElementCaptureManager::FComposureShotElementCaptureManager()
{
	ExportInitializerObject = TStrongObjectPtr<UMovieSceneComposureExportInitializer>(NewObject<UMovieSceneComposureExportInitializer>(GetTransientPackage()));
}

FComposureShotElementCaptureManager::~FComposureShotElementCaptureManager()
{
	for (TTuple<FObjectKey, FBoundPasses>& Pair : CompElementToPasses)
	{
		ACompositingElement* CompShotElement = Cast<ACompositingElement>(Pair.Key.ResolveObjectPtr());
		if (CompShotElement)
		{
			if (Pair.Value.OnFinalPassRenderedHandle.IsValid())
			{
				CompShotElement->OnFinalPassRendered.Remove(Pair.Value.OnFinalPassRenderedHandle);
			}

			if (Pair.Value.OnTransformPassRenderedHandle.IsValid())
			{
				CompShotElement->OnTransformPassRendered.Remove(Pair.Value.OnTransformPassRenderedHandle);
			}
		}
	}
}

void FComposureShotElementCaptureManager::CaptureShotElementPass(ACompositingElement* CompShotElement, const FMovieSceneComposureExportPass& InPass)
{
	check(CompShotElement);

	FObjectKey ShotElementKey(CompShotElement);

	FBoundPasses* BoundPasses = CompElementToPasses.Find(ShotElementKey);

	if (!BoundPasses)
	{
		BoundPasses = &CompElementToPasses.Add(ShotElementKey);

		BoundPasses->OnFinalPassRenderedHandle     = CompShotElement->OnFinalPassRendered.AddRaw(this, &FComposureShotElementCaptureManager::HandleOnFinalPassRendered);
		BoundPasses->OnTransformPassRenderedHandle = CompShotElement->OnTransformPassRendered.AddRaw(this, &FComposureShotElementCaptureManager::HandleOnTransformPassRendered);
		BoundPasses->SceneCaptureExtensions        = ExportInitializerObject->InitializeCompShotElement(CompShotElement);
	}

	BoundPasses->Passes.AddPass(InPass, CompShotElement);

	// If this is the main output, capture from all the scene capture extensions as well
	if (InPass.TransformPassName == NAME_None)
	{
		for (TSharedPtr<FExportIntermediateBuffersViewExtension, ESPMode::ThreadSafe> Extension : BoundPasses->SceneCaptureExtensions)
		{
			++Extension->NumOutstandingFrames;
		}
	}
}

void FComposureShotElementCaptureManager::StopCapturingShotElementPass(ACompositingElement* CompShotElement, FName PassName)
{
	check(CompShotElement);
	if (FBoundPasses* BoundPasses = CompElementToPasses.Find(CompShotElement))
	{
		BoundPasses->Passes.PassesToExport.Remove(PassName);
		if (BoundPasses->Passes.PassesToExport.Num() == 0)
		{
			CompShotElement->OnFinalPassRendered.Remove(BoundPasses->OnFinalPassRenderedHandle);
			CompShotElement->OnTransformPassRendered.Remove(BoundPasses->OnTransformPassRenderedHandle);

			CompElementToPasses.Remove(CompShotElement);
			// BoundPasses is now invalid
		}
	}
}

void FComposureShotElementCaptureManager::HandleOnTransformPassRendered(ACompositingElement* CompShotElement, UTexture* Texture, FName PassName)
{
	UUserDefinedCaptureProtocol* CaptureProtocol = Cast<UUserDefinedCaptureProtocol>(UMovieSceneCaptureEnvironment::FindImageCaptureProtocol());
	if (CaptureProtocol && CaptureProtocol->IsCapturing() && PassName != NAME_None)
	{
		ExportPass(CompShotElement, Texture, PassName);
	}
}

void FComposureShotElementCaptureManager::HandleOnFinalPassRendered(ACompositingElement* CompShotElement, UTexture* Texture)
{
	UUserDefinedCaptureProtocol* CaptureProtocol = Cast<UUserDefinedCaptureProtocol>(UMovieSceneCaptureEnvironment::FindImageCaptureProtocol());
	if (CaptureProtocol && CaptureProtocol->IsCapturing())
	{
		ExportPass(CompShotElement, Texture, NAME_None);
	}
}

void FComposureShotElementCaptureManager::ExportPass(ACompositingElement* CompShotElement, UTexture* Texture, FName PassName)
{
	UUserDefinedCaptureProtocol* CaptureProtocol = Cast<UUserDefinedCaptureProtocol>(UMovieSceneCaptureEnvironment::FindImageCaptureProtocol());

	check(CompShotElement && CaptureProtocol);

	FObjectKey ShotElementKey(CompShotElement);
	const FBoundPasses* BoundPasses = CompElementToPasses.Find(ShotElementKey);
	if (BoundPasses)
	{
		const FMovieSceneComposureExportPass* PassOptions = BoundPasses->Passes.PassesToExport.Find(PassName);
		if (PassOptions)
		{
			FCapturedPixelsID BufferID;

			BufferID.Identifiers.Add("Element", CompShotElement->GetFName());
			BufferID.Identifiers.Add("Pass", PassOptions->bRenamePass ? PassOptions->ExportedAs : PassName);

			CaptureProtocol->ResolveBuffer(Texture, BufferID);
		}
	}
}

