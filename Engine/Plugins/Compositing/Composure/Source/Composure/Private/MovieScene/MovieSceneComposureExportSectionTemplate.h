// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieScene/MovieSceneComposureExportTrack.h"
#include "MovieScene/MovieSceneComposureExportTrack.h"
#include "MovieSceneComposureExportSectionTemplate.generated.h"

struct FExportIntermediateBuffersViewExtension;

class USceneCaptureComponent2D;
class ACompositingElement;
class UMovieSceneComposureExportTrack;


/**
 * Object passed to comp shot elements to initialize them for export.
 * Currenly only allows scene captures to initialize a new extension that can capture GBuffers and other buffer visualization targets
 */
UCLASS()
class UMovieSceneComposureExportInitializer : public UObject
{
public:

	GENERATED_BODY()

	/**
	 * Initialize the export to capture the specified named buffer visualization targets from a scene capture
	 */
	UFUNCTION(BlueprintCallable, Category="Composure|Export")
	void ExportSceneCaptureBuffers(ACompositingElement* CompShotElement, USceneCaptureComponent2D* SceneCapture, const TArray<FString>& BuffersToExport);

	/** Implementation function */
	TArray<TSharedPtr<FExportIntermediateBuffersViewExtension, ESPMode::ThreadSafe>> InitializeCompShotElement(ACompositingElement* CompShotElement);

private:

	TArray<TSharedPtr<FExportIntermediateBuffersViewExtension, ESPMode::ThreadSafe>> TmpExtensions;
};



USTRUCT()
struct FMovieSceneComposureExportSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

	FMovieSceneComposureExportSectionTemplate() {}
	FMovieSceneComposureExportSectionTemplate(const UMovieSceneComposureExportTrack& Track);

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	UPROPERTY()
	FMovieSceneComposureExportPass Pass;
};
