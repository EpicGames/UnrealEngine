// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneComposureExportTrack.h"
#include "MovieScene/MovieSceneComposureExportSectionTemplate.h"

#define LOCTEXT_NAMESPACE "MovieSceneComposureExportTrack"

UMovieSceneComposureExportTrack::UMovieSceneComposureExportTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(0, 95, 133, 255);
#endif
}

bool UMovieSceneComposureExportTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{ 
	return SectionClass == UMovieSceneComposureExportSection::StaticClass(); 
}


FMovieSceneEvalTemplatePtr UMovieSceneComposureExportTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneComposureExportSectionTemplate(*this);
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneComposureExportTrack::GetDisplayName() const
{
	if (Pass.TransformPassName != NAME_None)
	{
		if (Pass.bRenamePass)
		{
			return FText::Format(LOCTEXT("RenamedPass_Format", "Export {0} [Internal - Source: {1}]"), FText::FromName(Pass.ExportedAs), FText::FromName(Pass.TransformPassName));
		}
		else 
		{
			return FText::Format(LOCTEXT("InternalPass_Format", "Export {0} [Internal]"), FText::FromName(Pass.TransformPassName));
		}
	}
	else
	{
		return LOCTEXT("DefaultName", "Export Output");
	}
}
#endif

UMovieSceneSection* UMovieSceneComposureExportTrack::CreateNewSection()
{
	return NewObject<UMovieSceneComposureExportSection>(this, NAME_None, RF_Transactional);
}

UMovieSceneComposureExportSection::UMovieSceneComposureExportSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
	bSupportsInfiniteRange = true;
	SetRange(TRange<FFrameNumber>::All());
}

#undef LOCTEXT_NAMESPACE
