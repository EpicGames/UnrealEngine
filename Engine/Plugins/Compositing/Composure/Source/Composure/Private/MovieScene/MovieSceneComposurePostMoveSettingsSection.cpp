// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneComposurePostMoveSettingsSection.h"
#include "MovieSceneCommonHelpers.h"
#include "UObject/SequencerObjectVersion.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "ComposurePostMoves.h"

LLM_DEFINE_TAG(Composure_MovieSceneComposurePostMoveSettingsSection);

#if WITH_EDITOR

struct FPostMoveSettingsChannelEditorData
{
	FPostMoveSettingsChannelEditorData()
	{
		FText PivotGroup = NSLOCTEXT("PostMoves", "Pivot", "Pivot");
		FText TranslationGroup = NSLOCTEXT("PostMoves", "Translation", "Translation");

		MetaData[0].SetIdentifiers("Pivot.X", FCommonChannelData::ChannelX, PivotGroup);
		MetaData[0].SubPropertyPath = MetaData[0].Name;
		MetaData[0].SortOrder = 0;
		MetaData[0].Color = FCommonChannelData::RedChannelColor;
		MetaData[0].bCanCollapseToTrack = false;

		MetaData[1].SetIdentifiers("Pivot.Y", FCommonChannelData::ChannelY, PivotGroup);
		MetaData[1].SubPropertyPath = MetaData[1].Name;
		MetaData[1].SortOrder = 1;
		MetaData[1].Color = FCommonChannelData::GreenChannelColor;
		MetaData[1].bCanCollapseToTrack = false;

		MetaData[2].SetIdentifiers("Translation.X", FCommonChannelData::ChannelX, TranslationGroup);
		MetaData[2].SubPropertyPath = MetaData[2].Name;
		MetaData[2].SortOrder = 2;
		MetaData[2].Color = FCommonChannelData::RedChannelColor;
		MetaData[2].bCanCollapseToTrack = false;

		MetaData[3].SetIdentifiers("Translation.Y", FCommonChannelData::ChannelY, TranslationGroup);
		MetaData[3].SubPropertyPath = MetaData[3].Name;
		MetaData[3].SortOrder = 3;
		MetaData[3].Color = FCommonChannelData::GreenChannelColor;
		MetaData[3].bCanCollapseToTrack = false;

		MetaData[4].SetIdentifiers("Rotation", NSLOCTEXT("PostMoves", "Rotation", "Rotation"));
		MetaData[4].SubPropertyPath = TEXT("RotationAngle");
		MetaData[4].SortOrder = 4;
		MetaData[4].bCanCollapseToTrack = false;

		MetaData[5].SetIdentifiers("Scale", NSLOCTEXT("PostMoves", "Scale", "Scale"));
		MetaData[5].SubPropertyPath = MetaData[5].Name;
		MetaData[5].SortOrder = 5;
		MetaData[5].bCanCollapseToTrack = false;

		ExternalValues[0].OnGetExternalValue = ExtractPivotX;
		ExternalValues[1].OnGetExternalValue = ExtractPivotY;
		ExternalValues[2].OnGetExternalValue = ExtractTranslationX;
		ExternalValues[3].OnGetExternalValue = ExtractTranslationY;
		ExternalValues[4].OnGetExternalValue = ExtractRotation;
		ExternalValues[5].OnGetExternalValue = ExtractScale;
	}

	static TOptional<float> ExtractPivotX(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FComposurePostMoveSettings>(InObject).Pivot.X : TOptional<float>();
	}
	static TOptional<float> ExtractPivotY(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FComposurePostMoveSettings>(InObject).Pivot.Y : TOptional<float>();
	}

	static TOptional<float> ExtractTranslationX(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FComposurePostMoveSettings>(InObject).Translation.X : TOptional<float>();
	}
	static TOptional<float> ExtractTranslationY(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FComposurePostMoveSettings>(InObject).Translation.Y : TOptional<float>();
	}

	static TOptional<float> ExtractRotation(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FComposurePostMoveSettings>(InObject).RotationAngle : TOptional<float>();
	}

	static TOptional<float> ExtractScale(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FComposurePostMoveSettings>(InObject).Scale : TOptional<float>();
	}

	FMovieSceneChannelMetaData MetaData[6];
	TMovieSceneExternalValue<float> ExternalValues[6];
};

#endif // WITH_EDITOR


UMovieSceneComposurePostMoveSettingsSection::UMovieSceneComposurePostMoveSettingsSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	LLM_SCOPE_BYTAG(Composure_MovieSceneComposurePostMoveSettingsSection);

	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);
	BlendType = EMovieSceneBlendType::Absolute;

	// Initialize this section's channel proxy
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	static const FPostMoveSettingsChannelEditorData EditorData;

	Channels.Add(Pivot[0],       EditorData.MetaData[0], EditorData.ExternalValues[0]);
	Channels.Add(Pivot[1],       EditorData.MetaData[1], EditorData.ExternalValues[1]);
	Channels.Add(Translation[0], EditorData.MetaData[2], EditorData.ExternalValues[2]);
	Channels.Add(Translation[1], EditorData.MetaData[3], EditorData.ExternalValues[3]);
	Channels.Add(RotationAngle,  EditorData.MetaData[4], EditorData.ExternalValues[4]);
	Channels.Add(Scale,          EditorData.MetaData[5], EditorData.ExternalValues[5]);

#else

	Channels.Add(Pivot[0]);
	Channels.Add(Pivot[1]);
	Channels.Add(Translation[0]);
	Channels.Add(Translation[1]);
	Channels.Add(RotationAngle);
	Channels.Add(Scale);

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}
