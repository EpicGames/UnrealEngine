// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneComposurePostMoveSettingsSectionTemplate.h"
#include "Evaluation/Blending/MovieSceneInitialValueStore.h"
#include "MovieScene/MovieSceneComposurePostMoveSettingsSection.h"
#include "Evaluation/MovieSceneExecutionTokens.h"
#include "Tracks/MovieScenePropertyTrack.h"

template<>
FMovieSceneAnimTypeID GetBlendingDataType<FComposurePostMoveSettings>()
{
	static FMovieSceneAnimTypeID TypeId = FMovieSceneAnimTypeID::Unique();
	return TypeId;
}

FMovieSceneComposurePostMoveSettingsSectionTemplate::FMovieSceneComposurePostMoveSettingsSectionTemplate(const UMovieSceneComposurePostMoveSettingsSection& Section, const UMovieScenePropertyTrack& Track)
	: FMovieScenePropertySectionTemplate(Track.GetPropertyName(), Track.GetPropertyPath().ToString())
	, BlendType(Section.GetBlendType().Get())
{
	Pivot[0] = Section.Pivot[0];
	Pivot[1] = Section.Pivot[1];

	Translation[0] = Section.Translation[0];
	Translation[1] = Section.Translation[1];

	RotationAngle = Section.RotationAngle;

	Scale = Section.Scale;
}

void FMovieSceneComposurePostMoveSettingsSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	const FFrameTime Time = Context.GetTime();
	UE::MovieScene::TMultiChannelValue<float, 6> AnimatedData;

	// Only activate channels if the curve has data associated with it
	auto EvalChannel = [&AnimatedData, Time](uint8 ChanneIndex, const FMovieSceneFloatChannel& Channel)
	{
		float Value = 0.f;
		if (Channel.Evaluate(Time, Value))
		{
			AnimatedData.Set(ChanneIndex, Value);
		}
	};

	EvalChannel(0, Pivot[0]);
	EvalChannel(1, Pivot[1]);

	EvalChannel(2, Translation[0]);
	EvalChannel(3, Translation[1]);

	EvalChannel(4, RotationAngle);

	EvalChannel(5, Scale);

	if (!AnimatedData.IsEmpty())
	{
		FMovieSceneBlendingActuatorID ActuatorTypeID = EnsureActuator<FComposurePostMoveSettings>(ExecutionTokens.GetBlendingAccumulator());

		// Add the blendable to the accumulator
		float Weight = EvaluateEasing(Context.GetTime());
		ExecutionTokens.BlendToken(ActuatorTypeID, TBlendableToken<FComposurePostMoveSettings>(AnimatedData, BlendType, Weight));
	}
}
