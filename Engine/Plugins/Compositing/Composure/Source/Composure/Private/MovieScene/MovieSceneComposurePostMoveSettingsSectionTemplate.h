// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelData.h"
#include "Evaluation/MovieScenePropertyTemplate.h"
#include "Evaluation/Blending/MovieSceneMultiChannelBlending.h"
#include "ComposurePostMoves.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneComposurePostMoveSettingsSectionTemplate.generated.h"

class UMovieSceneComposurePostMoveSettingsSection;
class UMovieScenePropertyTrack;

/** A movie scene evaluation template for post move settings sections. */
USTRUCT()
struct FMovieSceneComposurePostMoveSettingsSectionTemplate : public FMovieScenePropertySectionTemplate
{
	GENERATED_BODY()

	FMovieSceneComposurePostMoveSettingsSectionTemplate() : BlendType(EMovieSceneBlendType::Absolute) {}
	FMovieSceneComposurePostMoveSettingsSectionTemplate(const UMovieSceneComposurePostMoveSettingsSection& Section, const UMovieScenePropertyTrack& Track);

private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }

	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	UPROPERTY()
	FMovieSceneFloatChannel Pivot[2];

	UPROPERTY()
	FMovieSceneFloatChannel Translation[2];

	UPROPERTY()
	FMovieSceneFloatChannel RotationAngle;

	UPROPERTY()
	FMovieSceneFloatChannel Scale;

	UPROPERTY()
	EMovieSceneBlendType BlendType;
};

template<> COMPOSURE_API FMovieSceneAnimTypeID GetBlendingDataType<FComposurePostMoveSettings>();

template<> struct TBlendableTokenTraits<FComposurePostMoveSettings>
{
	typedef UE::MovieScene::TMaskedBlendable<float, 6> WorkingDataType;
};

namespace UE
{
namespace MovieScene
{
	inline void MultiChannelFromData(const FComposurePostMoveSettings& In, TMultiChannelValue<float, 6>& Out)
	{
		Out = { (float)In.Pivot.X, (float)In.Pivot.Y, (float)In.Translation.X, (float)In.Translation.Y, In.RotationAngle, In.Scale };
	}

	inline void ResolveChannelsToData(const TMultiChannelValue<float, 6>& In, FComposurePostMoveSettings& Out)
	{
		Out.Pivot = FVector2D(In[0], In[1]);
		Out.Translation = FVector2D(In[2], In[3]);
		Out.RotationAngle = In[4];
		Out.Scale = In[5];
	}

} // namespace MovieScene
} // namespace UE
