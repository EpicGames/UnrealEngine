// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneComposurePostMoveSettingsTrack.h"
#include "Channels/MovieSceneChannelData.h"
#include "MovieScene/MovieSceneComposurePostMoveSettingsSection.h"
#include "MovieSceneComposurePostMoveSettingsSectionTemplate.h"

UMovieSceneComposurePostMoveSettingsTrack::UMovieSceneComposurePostMoveSettingsTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(48, 227, 255, 65);
#endif
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneComposurePostMoveSettingsTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneComposurePostMoveSettingsSection::StaticClass();
}

UMovieSceneSection* UMovieSceneComposurePostMoveSettingsTrack::CreateNewSection()
{
	return NewObject<UMovieSceneComposurePostMoveSettingsSection>(this, NAME_None, RF_Transactional);
}


FMovieSceneEvalTemplatePtr UMovieSceneComposurePostMoveSettingsTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneComposurePostMoveSettingsSectionTemplate(*CastChecked<const UMovieSceneComposurePostMoveSettingsSection>(&InSection), *this);
}
