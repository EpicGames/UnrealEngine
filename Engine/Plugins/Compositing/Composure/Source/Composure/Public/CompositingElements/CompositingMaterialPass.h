// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture.h"
#include "UObject/ObjectMacros.h"

#include "CompositingMaterialPass.generated.h"

class UMaterialInterface;
class UMaterialInstanceDynamic;
class UTextureRenderTarget2D;

/* FCompositingParamPayload
 *****************************************************************************/

USTRUCT(BlueprintType)
struct COMPOSURE_API FCompositingParamPayload
{
	GENERATED_USTRUCT_BODY();

	/** Map of scalar material parameter names to their corresponding override values */
	UPROPERTY()
	TMap<FName, float> ScalarParamOverrides;

	/** Map of vector material parameter names to their corresponding override values */
	UPROPERTY()
	TMap<FName, FLinearColor> VectorParamOverrides;

	/** Map of texture material parameter names to their corresponding override values */
	UPROPERTY()
	TMap<FName, TObjectPtr<UTexture>> TextureParamOverrides;
};

/* FNamedCompMaterialParam
 *****************************************************************************/

UENUM()
enum class EParamType : uint8
{
	UnknownParamType,
	ScalarParam,
	VectorParam,
	TextureParam,
	MediaTextureParam,
};

USTRUCT(BlueprintType)
struct COMPOSURE_API FNamedCompMaterialParam
{
	GENERATED_USTRUCT_BODY();

	FNamedCompMaterialParam(const FName InParamName = NAME_None)
		: ParamName(InParamName)
	{}
	FNamedCompMaterialParam(const TCHAR* InParamName)
		: ParamName(InParamName)
	{}

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "CompositingMaterial")
	EParamType ParamType = EParamType::UnknownParamType;
#endif

	UPROPERTY(EditAnywhere, Category = "CompositingMaterial", meta = (DisplayName = "Default Parameter Name"))
	FName ParamName;


	operator FName()
	{
		return ParamName;
	}

	bool operator==(const FNamedCompMaterialParam& Rhs) const
	{ 
		return ParamName == Rhs.ParamName;
	}

	friend uint32 GetTypeHash(const FNamedCompMaterialParam& InNamedParam)
	{
		return GetTypeHash(InNamedParam.ParamName);
	}
};

/* FCompositingMaterial
 *****************************************************************************/

class ICompositingTextureLookupTable;
class UTexture;

USTRUCT(BlueprintType)
struct COMPOSURE_API FCompositingMaterial : public FCompositingParamPayload
{
	GENERATED_USTRUCT_BODY();

public:
	FCompositingMaterial();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "CompositingMaterial")
	TObjectPtr<UMaterialInterface> Material;

	/** Maps material texture param names to prior passes/elements. Overrides the element's param mapping list above. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "CompositingMaterial", meta = (ReadOnlyKeys, DisplayName = "Input Elements"))
	TMap<FName, FName> ParamPassMappings;

	UPROPERTY(EditDefaultsOnly, Category = "CompositingMaterial", meta = (DisplayName="Expected Param Mappings"))
	TMap<FName, FNamedCompMaterialParam> RequiredMaterialParams;

public:
	/** */
	bool ApplyParamOverrides(const ICompositingTextureLookupTable* TextureLookupTable);
	/** */
	void ResetMaterial();
	/** */
	void RenderToRenderTarget(UObject* WorldContext, UTextureRenderTarget2D* Target);

	/** */
	bool SetMaterialParam(const FName ParamName, float ScalarValue);
	bool SetMaterialParam(const FName ParamName, FLinearColor VectorValue);
	bool SetMaterialParam(const FName ParamName, UTexture* TextureValue);

public:
	void SetScalarOverride(const FName ParamName, const float ParamVal);
	bool GetScalarOverride(const FName ParamName, float& OutParamVal);
	void ResetScalarOverride(const FName ParamName);

	void SetVectorOverride(const FName ParamName, const FLinearColor ParamVal);
	bool GetVectorOverride(const FName ParamName, FLinearColor& OutParamVal);
	void ResetVectorOverride(const FName ParamName);

	/** Set the override UTexture value for the named input material parameter */
	void SetTextureOverride(const FName ParamName, UTexture* ParamVal);

	/** Get the override UTexture value for the named input material parameter */
	bool GetTextureOverride(const FName ParamName, UTexture*& OutParamVal);

	/** Remove the override for the named input material parameter and reset its value to the default */
	void ResetTextureOverride(const FName ParamName);

	void ResetAllParamOverrides();

	void MarkDirty() { bParamsModified = true; };

	/** */
	UMaterialInstanceDynamic* GetMID();

public:
//~ Editor-Only/Centric settings (used by the details panel customization)
#if WITH_EDITORONLY_DATA
	UPROPERTY(EditDefaultsOnly, Category = "CompositingMaterial|Editor")
	TArray<FName> EditorHiddenParams;

	/** */
	void UpdateProxyMap();

private:
	/** Required for customizing the color picker widget - need a property to wrap (one for each material param). */
	UPROPERTY(EditAnywhere, Category = "CompositingMaterial|Editor", meta=(ReadOnlyKeys))
	TMap<FName, FLinearColor> VectorOverrideProxies;
#endif // WITH_EDITORONLY_DATA 

private:
	bool bParamsModified = true;

	UPROPERTY(Transient, SkipSerialization)
	TObjectPtr<UMaterialInstanceDynamic> CachedMID;
};

