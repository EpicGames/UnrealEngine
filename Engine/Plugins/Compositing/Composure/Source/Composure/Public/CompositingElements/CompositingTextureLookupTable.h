// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CompositingElements/ICompositingTextureLookupTable.h"

/* FCompositingTextureLookupTable
 *****************************************************************************/

class FCompositingTextureLookupTable : public ICompositingTextureLookupTable
{
public:
	virtual ~FCompositingTextureLookupTable() {}

	void RegisterPassResult(FName KeyName, UTexture* Result, const int32 UsageTags = 0x00);
	void SetMostRecentResult(UTexture* Result);

	void ResetAll();
	void Empty(const int32 KeepTags = 0x00);
	void ClearTaggedEntries(const int32 UsageTags, const bool bRemove = false);
	void Remove(UTexture* Texture);

	void LinkNestedSearchTable(FName KeyName, ICompositingTextureLookupTable* NestedLookupTable);
	void ClearLinkedSearchTables();

	int32 FindUsageTags(FName LookupName);

public:
	//~ Begin ICompositingTextureLookupTable interface
	virtual bool FindNamedPassResult(FName LookupName, UTexture*& OutTexture) const override;
	bool FindNamedPassResult(FName LookupName, bool bSearchLinkedTables, UTexture*& OutTexture) const;
	//~ End ICompositingTextureLookupTable interface

private:
	struct FTaggedTexture
	{
		int32 UsageTags = 0x00;
		UTexture* Texture = nullptr;
	};
	typedef TMap<FName, FTaggedTexture> FInternalLookupTable;
	FInternalLookupTable LookupTable;
	
	TMap<FName, ICompositingTextureLookupTable*> LinkedSearchTables;

public:

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	FORCEINLINE FInternalLookupTable::TRangedForIterator      begin()       { return LookupTable.begin(); }
	FORCEINLINE FInternalLookupTable::TRangedForConstIterator begin() const { return LookupTable.begin(); }
	FORCEINLINE FInternalLookupTable::TRangedForIterator      end  ()       { return LookupTable.end(); }
	FORCEINLINE FInternalLookupTable::TRangedForConstIterator end  () const { return LookupTable.end(); }
};