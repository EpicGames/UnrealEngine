// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/TextureRenderTarget2D.h" // for ETextureRenderTargetFormat
#include "UObject/WeakObjectPtr.h"

class UTextureRenderTarget2D;
class UObject;
class FCompElementRenderTargetPool;

/* FInheritedTargetPool
 *****************************************************************************/

typedef TSharedPtr<FCompElementRenderTargetPool> FSharedTargetPoolPtr;
typedef TWeakPtr<FCompElementRenderTargetPool> FWeakTargetPoolPtr;

struct FInheritedTargetPool
{
public:
	FInheritedTargetPool(UObject* Owner, FIntPoint NativeTargetResolution, ETextureRenderTargetFormat NativeTargetFormat, const FSharedTargetPoolPtr& InheritedPool, int32 UsageTags = 0x00);
	FInheritedTargetPool() {}
	FInheritedTargetPool(FInheritedTargetPool& Other, FIntPoint NewTargetResolution, ETextureRenderTargetFormat NewTargetFormat);

	bool IsValid() const;
	void Reset();

	UTextureRenderTarget2D* RequestRenderTarget(float RenderScale = 1.f);
	UTextureRenderTarget2D* RequestRenderTarget(FIntPoint Dimensions, ETextureRenderTargetFormat Format);
	bool ReleaseRenderTarget(UTextureRenderTarget2D* UsedTarget);

private:
	FWeakTargetPoolPtr InheritedPool;
	TWeakObjectPtr<UObject> Owner;

	friend struct FScopedTargetPoolTagAddendum;
	int32 UsageTags = 0x00;

	FIntPoint NativeTargetResolution = FIntPoint(1920.f, 1080.f);
	ETextureRenderTargetFormat NativeTargetFormat = RTF_RGBA16f;
};

/* FScopedTargetPoolTagAddendum
 *****************************************************************************/

struct FScopedTargetPoolTagAddendum
{
public:
	 FScopedTargetPoolTagAddendum(int32 NewTags, FInheritedTargetPool& TargetPool);
	~FScopedTargetPoolTagAddendum();

private:
	FInheritedTargetPool& TargetPool;
	int32 TagsToRestore = 0x00;
};
