// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "ComposurePlayerCompositingInterface.generated.h"

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UComposurePlayerCompositingInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class COMPOSURE_API IComposurePlayerCompositingInterface
{
	GENERATED_IINTERFACE_BODY()

	// Entries called by PlayerCameraModifier.
	virtual void OverrideBlendableSettings(class FSceneView& View, float Weight) const {};
};
