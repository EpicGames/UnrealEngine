// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComposurePostMoves.generated.h"


USTRUCT(BlueprintType)
struct COMPOSURE_API FComposurePostMoveSettings
{
	GENERATED_USTRUCT_BODY()
	FComposurePostMoveSettings()
	{
		Pivot = FVector2D(0.5, 0.5);
		Translation = FVector2D(0, 0);
		RotationAngle = 0;
		Scale = 1;
	}


	/** The normalized pivot point for applying rotation and scale to the image. The x and y values are normalized to the range 0-1 where 1 represents the full width and height of the image. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Moves")
	FVector2D Pivot;

	/** The translation to apply to the image.  The x and y values are normalized to the range 0-1 where 1 represents the full width and height of the image. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Moves")
	FVector2D Translation;

	/** The anti clockwise rotation to apply to the image in degrees. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Moves")
	float RotationAngle;

	/** The scale to apply to the image. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Post Moves")
	float Scale;

	
	/** Returns a non-centered projection matrix.
	 * @param HorizontalFOVAngle The desired horizontal FOV in degrees.
	 * @param AspectRatio The desired aspect ratio.
	 */
	FMatrix GetProjectionMatrix(float HorizontalFOVAngle, float AspectRatio) const;
	
	/** Returns UV transformation matrix and its inversed to crop.
	 * @param AspectRatio The desired aspect ratio.
	 */
	void GetCroppingUVTransformationMatrix(
		float AspectRatio,
		FMatrix* OutCropingUVTransformationMatrix,
		FMatrix* OutUncropingUVTransformationMatrix) const;
};
