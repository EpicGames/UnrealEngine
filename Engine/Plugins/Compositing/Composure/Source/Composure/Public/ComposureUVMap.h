// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "ComposureUVMap.generated.h"


USTRUCT(BlueprintType)
struct COMPOSURE_API FComposureUVMapSettings
{
	GENERATED_USTRUCT_BODY()
	FComposureUVMapSettings()
		: PreUVDisplacementMatrix(FMatrix::Identity)
		, PostUVDisplacementMatrix(FMatrix::Identity)
		, DisplacementDecodeParameters(FVector2D(1, 0))
		, DisplacementTexture(nullptr)
		, bUseDisplacementBlueAndAlphaChannels(false)
	{ }


	/** UV Matrix to apply before sampling DisplacementTexture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UV Mapping")
	FMatrix PreUVDisplacementMatrix;

	/** UV Matrix to apply after displacing UV using DisplacementTexture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UV Mapping")
	FMatrix PostUVDisplacementMatrix;

	/** Decoding parameters for DisplacementTexture. DeltaUV = ((RedChannel, GreenChannel) - Y) * X. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UV Mapping")
	FVector2D DisplacementDecodeParameters;

	/** Displacement texture to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UV Mapping")
	TObjectPtr<class UTexture> DisplacementTexture;

	/** Whether to use blue and alpha channel instead of red and green channel in computation of DeltaUV. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "UV Mapping")
	uint32 bUseDisplacementBlueAndAlphaChannels : 1;


	/** Sets parameters of a material that uses MF_UVMap_SampleLocation material function. */
	void SetMaterialParameters(class UMaterialInstanceDynamic* MID) const;


	/**
	 * Converts displacement encoding parameters to decoding parameters.
	 * Can also be used to convert displacement decoding parameters to encoding parameters.
	 */
	static inline FVector2D InvertEncodingParameters(FVector2D EncodingParameters)
	{
		return FVector2D(1.f / EncodingParameters.X, EncodingParameters.Y);
	}
};
