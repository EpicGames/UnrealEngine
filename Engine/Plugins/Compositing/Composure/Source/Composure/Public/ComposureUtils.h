// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


struct FComposureUtils
{

/* Disables the showflags to a scene capture post processing only. */
COMPOSURE_API static void SetEngineShowFlagsForPostprocessingOnly(struct FEngineShowFlags& InOutEngineShowflags);


/** Returns the red and green channel fringe factors from percentage of chromatic aberration. */
COMPOSURE_API static FVector2D GetRedGreenUVFactorsFromChromaticAberration(float ChromaticAberrationAmount);

};
