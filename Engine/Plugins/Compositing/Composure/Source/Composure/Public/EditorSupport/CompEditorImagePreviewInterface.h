// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "CompEditorImagePreviewInterface.generated.h"

class UTexture;

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UCompEditorImagePreviewInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class COMPOSURE_API ICompEditorImagePreviewInterface
{
	GENERATED_IINTERFACE_BODY()

public:
#if WITH_EDITOR
	virtual void OnBeginPreview() {};
	virtual UTexture* GetEditorPreviewImage() = 0;
	virtual void OnEndPreview() {};
	virtual bool UseImplicitGammaForPreview() const = 0;
#endif
};
