// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/CoreMiscDefines.h" // for EForceInit

enum class ETargetUsageFlags : uint8;
typedef FGuid FFreezeFrameControlHandle;

struct FCompFreezeFrameController
{
public:
	FCompFreezeFrameController(int32& FreezeFlagsRef)
		: FreezeFlags(FreezeFlagsRef)
	{}

	FORCEINLINE bool IsLocked() const 
	{ 
		return LockKey.IsValid(); 
	}
	
	FFreezeFrameControlHandle Lock()
	{ 
		if (ensure(!IsLocked()))
		{
			LockKey = FGuid::NewGuid();
			return LockKey;
		}
		return FGuid();
	}

	FORCEINLINE bool Unlock(const FFreezeFrameControlHandle& InLockKey)
	{ 
		if (InLockKey == LockKey)
		{
			LockKey.Invalidate();
		}
		return !IsLocked();
	}

	FORCEINLINE ETargetUsageFlags GetFreezeFlags() const { return (ETargetUsageFlags)FreezeFlags; }

	COMPOSURE_API bool SetFreezeFlags(ETargetUsageFlags InFreezeFlags, bool bClearOthers = false, const FFreezeFrameControlHandle& LockKey = FFreezeFrameControlHandle());
	COMPOSURE_API bool ClearFreezeFlags(ETargetUsageFlags InFreezeFlags, const FFreezeFrameControlHandle& LockKey = FFreezeFrameControlHandle());
	COMPOSURE_API bool ClearFreezeFlags(const FFreezeFrameControlHandle& LockKey = FFreezeFrameControlHandle());

	FORCEINLINE operator ETargetUsageFlags() const { return GetFreezeFlags(); }
	FORCEINLINE operator int32()                   { return (int32)GetFreezeFlags(); }

	COMPOSURE_API bool HasAnyFlags(ETargetUsageFlags InFreezeFlags);
	COMPOSURE_API bool HasAllFlags(ETargetUsageFlags InFreezeFlags);

private: 
	FFreezeFrameControlHandle LockKey;
	int32& FreezeFlags;

public:

	/** DO NOT USE - For UObject construction only */
	COMPOSURE_API FCompFreezeFrameController(EForceInit Default = ForceInit);
};

