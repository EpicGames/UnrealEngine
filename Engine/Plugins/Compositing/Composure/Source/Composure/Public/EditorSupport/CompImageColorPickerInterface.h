// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "EditorSupport/CompEditorImagePreviewInterface.h"
#include "CompImageColorPickerInterface.generated.h"

class UTextureRenderTarget2D;
struct FCompFreezeFrameController;

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UCompImageColorPickerInterface : public UCompEditorImagePreviewInterface
{
	GENERATED_UINTERFACE_BODY()
};

class COMPOSURE_API ICompImageColorPickerInterface : public ICompEditorImagePreviewInterface
{
	GENERATED_IINTERFACE_BODY()

public:
#if WITH_EDITOR
	virtual UTexture* GetColorPickerDisplayImage() = 0;
	virtual UTextureRenderTarget2D* GetColorPickerTarget() = 0;
	virtual FCompFreezeFrameController* GetFreezeFrameController() = 0;
#endif
};
