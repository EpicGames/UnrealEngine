// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"
#include "EditorSupport/WeakUInterfacePtr.h"
#include "EditorSupport/CompImageColorPickerInterface.h"

class UTexture;
class SWidget;
class SWindow;
class ICompImageColorPickerInterface;
class ACompositingElement;

class COMPOSURE_API ICompositingEditor : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("ComposureCompositingEditor"));
		return FeatureName;
	}
	static ICompositingEditor* Get();

	/** */
	DECLARE_DELEGATE_RetVal(UTexture*, FGetPreviewTexture);
	virtual TSharedPtr<SWidget> ConstructCompositingPreviewPane(TWeakUIntrfacePtr<ICompEditorImagePreviewInterface> PreviewTarget) = 0;

	/** */
	DECLARE_DELEGATE_ThreeParams(FPickerResultHandler, const FVector2D& /*PickedUV*/, const FLinearColor& /*PickedColor*/, bool /*bInteractive*/);
	virtual TSharedPtr<SWindow> RequestCompositingPickerWindow(TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget, const bool bAverageColorOnDrag, const FPickerResultHandler& OnPick, const FSimpleDelegate& OnCancel, const FText& WindowTitle) = 0;

	/** */
	virtual bool DeferCompositingDraw(ACompositingElement* CompElement) = 0;
	virtual void RequestRedraw() = 0;
};
