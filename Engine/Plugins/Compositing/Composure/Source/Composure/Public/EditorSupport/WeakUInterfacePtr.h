// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/Casts.h"

class UObject;

template<class T>
class TWeakUIntrfacePtr
{
public:
	TWeakUIntrfacePtr(UObject* InObject = nullptr)
		: Object(InObject)
	{
		IntrefacePtr = Cast<T>(Object);
	}

	template<class U>
	TWeakUIntrfacePtr(const TWeakUIntrfacePtr<U>& Rhs)
	{
		IntrefacePtr = Cast<T>(Rhs.Get());
		Object = Rhs.GetObject();
	}

	FORCEINLINE bool IsValid() const
	{
		return Object.IsValid() && IntrefacePtr;
	}

	FORCEINLINE T* Get() const
	{
		return IsValid() ? IntrefacePtr : nullptr;
	}

	FORCEINLINE T& operator*() const
	{
		check(IsValid());
		return *Get();
	}

	FORCEINLINE T* operator->() const
	{
		check(IsValid());
		return Get();
	}

	FORCEINLINE void Reset()
	{
		IntrefacePtr = nullptr;
		Object.Reset();
	}

	FORCEINLINE TWeakObjectPtr<UObject> GetObject() const
	{
		return Object;
	}

private:
	T* IntrefacePtr;
	TWeakObjectPtr<UObject> Object;
};
