// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "Protocols/UserDefinedCaptureProtocol.h"
#include "Protocols/CompositionGraphCaptureProtocol.h"
#include "IMovieSceneComposureExportClient.generated.h"

class UTexture;
class ACompositingElement;


UINTERFACE(Category="Sequencer", Blueprintable, meta=(DisplayName = "Composure Export Client"))
class COMPOSURE_API UMovieSceneComposureExportClient : public UInterface
{
public:
	GENERATED_BODY()
};

/**
 * Interface that can be implemented for any custom composure element to initialize it for export.
 */
class COMPOSURE_API IMovieSceneComposureExportClient
{
public:
	GENERATED_BODY()

	/**
	 * Initialize this object for export by setting up any of the necessary scene view extensions with the specified initializer.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category="Composure|Export")
	void InitializeForExport(UMovieSceneComposureExportInitializer* ExportInitializer);
};