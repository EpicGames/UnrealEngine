// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "MovieSceneComposureExportTrack.generated.h"

/**
 * Export configuration options for a single internal pass on an ACompositingElement, or its output pass (where TransformPassName is None)
 */
USTRUCT()
struct FMovieSceneComposureExportPass
{
	GENERATED_BODY()

	/** The name of the transform pass in the comp to export. None signifies the element's output. */
	UPROPERTY(EditAnywhere, Category="Export")
	FName TransformPassName;

	/** Whether to rename this pass when rendering out */
	UPROPERTY(EditAnywhere, Category="Export", meta=(InlineEditConditionToggle))
	bool bRenamePass = false;

	/** The name to use for this pass when rendering out */
	UPROPERTY(EditAnywhere, Category="Export", meta=(EditCondition="bRenamePass"))
	FName ExportedAs;
};


/**
 * Movie scene track that exports a single pass (either the element's output, or an internal transform pass) during burnouts
 */
UCLASS(MinimalAPI)
class UMovieSceneComposureExportTrack : public UMovieSceneTrack, public IMovieSceneTrackTemplateProducer
{
public:
	GENERATED_BODY()

	/**
	 * Configuration options for the pass to export
	 */
	UPROPERTY(EditAnywhere, Category="Export", meta=(ShowOnlyInnerProperties))
	FMovieSceneComposureExportPass Pass;


	UMovieSceneComposureExportTrack(const FObjectInitializer& ObjectInitializer);


	virtual UMovieSceneSection* CreateNewSection() override;

	virtual bool IsEmpty() const override { return Sections.Num() == 0; }
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual void AddSection(UMovieSceneSection& Section) override              { Sections.AddUnique(&Section); }
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override { return Sections; }
	virtual bool HasSection(const UMovieSceneSection& Section) const override  { return Sections.Contains(&Section); }
	virtual void RemoveSection(UMovieSceneSection& Section) override { Sections.Remove(&Section); }
	virtual void RemoveSectionAt(int32 SectionIndex) override { Sections.RemoveAt(SectionIndex); }
	virtual bool SupportsMultipleRows() const override                         { return true; }
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
#endif

private:

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};

UCLASS(MinimalAPI)
class UMovieSceneComposureExportSection : public UMovieSceneSection
{
public:
	GENERATED_BODY()

	UMovieSceneComposureExportSection(const FObjectInitializer& ObjectInitializer);
};