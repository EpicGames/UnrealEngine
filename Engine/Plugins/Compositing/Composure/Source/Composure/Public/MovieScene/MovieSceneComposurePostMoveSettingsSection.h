// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneComposurePostMoveSettingsSection.generated.h"

/**
* A movie scene section for animating FComposurePostMoveSettings properties.
*/
UCLASS(MinimalAPI)
class UMovieSceneComposurePostMoveSettingsSection
	: public UMovieSceneSection
{
	GENERATED_UCLASS_BODY()

public:

	/** The curves for animating the pivot property. */
	UPROPERTY()
	FMovieSceneFloatChannel Pivot[2];

	/** The curves for animating the translation property. */
	UPROPERTY()
	FMovieSceneFloatChannel Translation[2];

	/** The curve for animating the rotation angle property. */
	UPROPERTY()
	FMovieSceneFloatChannel RotationAngle;

	/** The curve for animating the scale property. */
	UPROPERTY()
	FMovieSceneFloatChannel Scale;
};
