// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tracks/MovieScenePropertyTrack.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "MovieSceneComposurePostMoveSettingsTrack.generated.h"

/**
* A track for animating FComposurePostMoveSettings properties.
*/
UCLASS(MinimalAPI)
class UMovieSceneComposurePostMoveSettingsTrack : public UMovieScenePropertyTrack, public IMovieSceneTrackTemplateProducer
{
	GENERATED_BODY()

public:

	UMovieSceneComposurePostMoveSettingsTrack(const FObjectInitializer& ObjectInitializer);

	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
};