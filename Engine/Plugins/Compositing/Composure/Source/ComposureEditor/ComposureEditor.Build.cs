// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ComposureEditor : ModuleRules
	{
		public ComposureEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
			});

			PrivateDependencyModuleNames.AddRange(new string[] {
				"Composure",
				"Layers",
				"EditorWidgets",
				"MovieScene",
				"MovieSceneTracks",
				"MovieSceneTools",
				"Sequencer",
				"Slate",
				"SlateCore",
				"TimeManagement",
				"DesktopWidgets",
				"EditorFramework",
				"UnrealEd"
			});
		}
	}
}
