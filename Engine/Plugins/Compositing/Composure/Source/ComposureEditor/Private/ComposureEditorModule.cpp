// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureEditorModule.h"
#include "Modules/ModuleManager.h"
#include "ISequencerModule.h"
#include "MovieScene/MovieSceneComposureExportTrack.h"
#include "Sequencer/ComposurePostMoveSettingsPropertyTrackEditor.h"
#include "Sequencer/ComposureExportTrackEditor.h"

LLM_DEFINE_TAG(Composure_ComposureEditor);
DEFINE_LOG_CATEGORY(LogComposureEditor);

class FComposureEditorModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(Composure_ComposureEditor);

		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		CreatePostMoveSettingsPropertyTrackEditorHandle = SequencerModule.RegisterPropertyTrackEditor<FComposurePostMoveSettingsPropertyTrackEditor>();
		ComposureExportTrackEditorHandle               = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateLambda([](TSharedRef<ISequencer> In){ return MakeShared<FComposureExportTrackEditor>(In); }));
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE_BYTAG(Composure_ComposureEditor);

		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule != nullptr)
		{
			SequencerModule->UnRegisterTrackEditor(CreatePostMoveSettingsPropertyTrackEditorHandle);
			SequencerModule->UnRegisterTrackEditor(ComposureExportTrackEditorHandle);
		}
	}

private:

	FDelegateHandle CreatePostMoveSettingsPropertyTrackEditorHandle;
	FDelegateHandle ComposureExportTrackEditorHandle;
};

IMPLEMENT_MODULE(FComposureEditorModule, ComposureEditor )
