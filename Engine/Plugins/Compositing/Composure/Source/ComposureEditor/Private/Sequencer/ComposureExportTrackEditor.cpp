// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureExportTrackEditor.h"
#include "CompositingElement.h"
#include "CompositingElements/CompositingElementPasses.h"


#include "DetailsViewArgs.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "ISequencer.h"
#include "Modules/ModuleManager.h"
#include "MovieScene/MovieSceneComposureExportTrack.h"
#include "SequencerSettings.h"

#define LOCTEXT_NAMESPACE "ComposureExportTrackEditor"


FComposureExportTrackEditor::FComposureExportTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FKeyframeTrackEditor(InSequencer)
{
	OnActorAddedToSequencerHandle = InSequencer->OnActorAddedToSequencer().AddRaw(this, &FComposureExportTrackEditor::HandleActorAdded);
}

FComposureExportTrackEditor::~FComposureExportTrackEditor()
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		SequencerPtr->OnActorAddedToSequencer().Remove(OnActorAddedToSequencerHandle);
	}
}

void FComposureExportTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	UMovieSceneComposureExportTrack* ExportTrack = Cast<UMovieSceneComposureExportTrack>(Track);

	auto PopulateSubMenu = [this, ExportTrack](FMenuBuilder& SubMenuBuilder)
	{
		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Create a details view for the track
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.ColumnWidth = 0.55f;

		TSharedRef<IDetailsView> DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);

		// Assign the object
		DetailsView->SetObject(ExportTrack, true);

		TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
		const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
		const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

		// Add it to the menu
		TSharedRef< SWidget > DetailsViewWidget =
			SNew(SBox)
			.WidthOverride(WidthOverride)
			.HeightOverride(HeightOverride)
		[
			DetailsView
		];

		SubMenuBuilder.AddWidget(DetailsViewWidget, FText(), true, false);
	};

	MenuBuilder.AddSubMenu(LOCTEXT("Properties_MenuText", "Properties"), FText(), FNewMenuDelegate::CreateLambda(PopulateSubMenu));
}

FText FComposureExportTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ComposureExportTrackEditor_DisplayName", "Composure Export");
}

void FComposureExportTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();

	TSet<FName> ExistingPasses;
	{
		const FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectBindings[0]);
		if (Binding)
		{
			for (UMovieSceneTrack* Track : Binding->GetTracks())
			{
				UMovieSceneComposureExportTrack* ExportTrack = Cast<UMovieSceneComposureExportTrack>(Track);
				if (ExportTrack)
				{
					ExistingPasses.Add(ExportTrack->Pass.TransformPassName);
				}
			}
		}
	}

	for (TWeakObjectPtr<> WeakObject : SequencerPtr->FindObjectsInCurrentSequence(ObjectBindings[0]))
	{
		ACompositingElement* CompShotElement = Cast<ACompositingElement>(WeakObject.Get());
		if (CompShotElement)
		{
			if (!ExistingPasses.Contains(NAME_None))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("AddOutputTrack_Label", "Export Output"),
					LOCTEXT("AddOutputTrack_Tooltip", "Adds a new track that controls whether this composure element's output should be captured as part of a Sequencer capture."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw(this, &FComposureExportTrackEditor::AddNewTrack, ObjectBindings, FName(), true, FName("Output"))
					)
				);
			}

			// Add all this comp element's transform passes as sections in order
			TArray<UCompositingElementTransform*> TransformPasses = CompShotElement->GetTransformsList();
			for (int32 Index = 0; Index < TransformPasses.Num(); ++Index)
			{
				UCompositingElementTransform* TransformPass = TransformPasses[Index];

				if (TransformPass && !ExistingPasses.Contains(TransformPass->PassName))
				{
					MenuBuilder.AddMenuEntry(
						FText::Format(LOCTEXT("AddTrack_Label", "Export Transform Pass '{0}'"), FText::FromName(TransformPass->PassName)),
						LOCTEXT("AddTrack_Tooltip", "Adds a new track that controls whether this composure element pass should be captured as part of a Sequencer capture."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateRaw(this, &FComposureExportTrackEditor::AddNewTrack, ObjectBindings, TransformPass->PassName, false, FName())
						)
					);
				}
			}
			
			MenuBuilder.AddMenuSeparator();
		}
	}
}

void FComposureExportTrackEditor::AddNewTrack(TArray<FGuid> ObjectBindings, FName InPassName, bool bRenamePass, FName InExportAs)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("AddNewTrack_Transaction", "Add Composure Export Track"));

		for (FGuid ObjectBinding : ObjectBindings)
		{
			UMovieScene*                       MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
			UMovieSceneComposureExportTrack*   NewTrack = MovieScene->AddTrack<UMovieSceneComposureExportTrack>(ObjectBinding);
			UMovieSceneComposureExportSection* NewSection = Cast<UMovieSceneComposureExportSection>(NewTrack->CreateNewSection());

			NewTrack->Pass.TransformPassName = InPassName;
			NewTrack->Pass.bRenamePass = bRenamePass;
			NewTrack->Pass.ExportedAs = InExportAs;

			NewTrack->AddSection(*NewSection);
		}

		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}

void FComposureExportTrackEditor::HandleActorAdded(AActor* Actor, FGuid TargetObjectGuid)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	ACompositingElement* CompShotElement = Cast<ACompositingElement>(Actor);
	if (CompShotElement && SequencerPtr.IsValid())
	{
		UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();

		// Add the main element output
		UMovieSceneComposureExportTrack*   NewTrack   = MovieScene->AddTrack<UMovieSceneComposureExportTrack>(TargetObjectGuid);
		UMovieSceneComposureExportSection* NewSection = Cast<UMovieSceneComposureExportSection>(NewTrack->CreateNewSection());

		NewTrack->Pass.bRenamePass = true;
		NewTrack->Pass.ExportedAs  = "Output";

		NewTrack->AddSection(*NewSection);
	}
}

bool FComposureExportTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	return true;
}

bool FComposureExportTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneComposureExportTrack::StaticClass();
}

#undef LOCTEXT_NAMESPACE
