// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KeyframeTrackEditor.h"
#include "MovieScene/MovieSceneComposureExportTrack.h"

class FComposureExportTrackEditor : public FKeyframeTrackEditor<UMovieSceneComposureExportTrack>
{
public:

	FComposureExportTrackEditor(TSharedRef<ISequencer> InSequencer);
	~FComposureExportTrackEditor();

private:

	virtual FText GetDisplayName() const override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;

	void HandleActorAdded(AActor* Actor, FGuid TargetObjectGuid);
	void AddNewTrack(TArray<FGuid> ObjectBindings, FName InPassName, bool bRenamePass, FName ExportAs);

	FDelegateHandle OnActorAddedToSequencerHandle;
};
