// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposurePostMoveSettingsPropertyTrackEditor.h"
#include "MovieScene/MovieSceneComposurePostMoveSettingsSection.h"
#include "ComposurePostMoves.h"
#include "SComposurePostMoveSettingsImportDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/FileHelper.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ComposureEditorModule.h"

#define LOCTEXT_NAMESPACE "ComposurePostMoveSettingsPropertyTrackEditor"

TSharedRef<ISequencerTrackEditor> FComposurePostMoveSettingsPropertyTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FComposurePostMoveSettingsPropertyTrackEditor(InSequencer));
}

FText FComposurePostMoveSettingsPropertyTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ComposurePostMoveSettingsPropertyTrackEditor_DisplayName", "Composure Post Move Settings Property");
}

void FComposurePostMoveSettingsPropertyTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track)
{
	MenuBuilder.BeginSection("PostMoveSettings", NSLOCTEXT("PostMoveSettingsTrackEditor", "PostMoveSettingsMenuSection", "Post Move Settings"));
	{
		UMovieSceneComposurePostMoveSettingsTrack* PostMoveSettingsTrack = CastChecked<UMovieSceneComposurePostMoveSettingsTrack>(Track);
		MenuBuilder.AddMenuEntry(
			NSLOCTEXT("PostMoveSettingsTrackEditor", "ImportPostMoveSettings", "Import from file..."),
			NSLOCTEXT("PostMoveSettingsTrackEditor", "ImportPostMoveSettingsToolTip", "Shows a dialog used to import post move track data from an external file."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FComposurePostMoveSettingsPropertyTrackEditor::ShowImportPostMoveSettingsDialog, PostMoveSettingsTrack)));
	}
	MenuBuilder.EndSection();
	
	MenuBuilder.AddMenuSeparator();
	FPropertyTrackEditor::BuildTrackContextMenu(MenuBuilder, Track);
}


void FComposurePostMoveSettingsPropertyTrackEditor::ShowImportPostMoveSettingsDialog(UMovieSceneComposurePostMoveSettingsTrack* PostMoveSettingsTrack)
{
	UMovieScene* ParentMovieScene = PostMoveSettingsTrack->GetTypedOuter<UMovieScene>();
	if (ParentMovieScene != nullptr)
	{
		TRange<FFrameNumber> PlaybackRange = ParentMovieScene->GetPlaybackRange();
		const FFrameNumber   StartFrame    = PlaybackRange.GetLowerBound().IsInclusive() ? PlaybackRange.GetLowerBoundValue() : PlaybackRange.GetLowerBoundValue() + 1;

		TSharedPtr<SWindow> TopLevelWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
		if (TopLevelWindow.IsValid())
		{
			TSharedRef<SWindow> Dialog =
				SNew(SComposurePostMoveSettingsImportDialog, ParentMovieScene->GetDisplayRate(), StartFrame)
				.OnImportSelected(this, &FComposurePostMoveSettingsPropertyTrackEditor::ImportPostMoveSettings, PostMoveSettingsTrack)
				.OnImportCanceled(this, &FComposurePostMoveSettingsPropertyTrackEditor::ImportCanceled);
			FSlateApplication::Get().AddWindowAsNativeChild(Dialog, TopLevelWindow.ToSharedRef());
			ImportDialog = Dialog;
		}
	}
}


void NotifyImportFailed(FString Path, FText Message)
{
	FText FormattedMessage = FText::Format(
		LOCTEXT("NotifyImportFailedFormat", "Failed to import {0}.  Message: {1}"),
		FText::FromString(Path),
		Message);

	// Write to log.
	UE_LOGF(LogComposureEditor, Warning, "%ls", *FormattedMessage.ToString());

	// Show toast.
	FNotificationInfo Info(FormattedMessage);
	Info.ExpireDuration = 5.0f;
	Info.bFireAndForget = true;
	Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
	FSlateNotificationManager::Get().AddNotification(Info);
}


void FComposurePostMoveSettingsPropertyTrackEditor::ImportPostMoveSettings(FString ImportFilePath, FFrameRate ImportFrameRate, FFrameNumber StartFrame, UMovieSceneComposurePostMoveSettingsTrack* PostMoveSettingsTrack)
{
	TSharedPtr<SWindow> DialogPinned = ImportDialog.Pin();
	if (DialogPinned.IsValid())
	{
		FSlateApplication::Get().RequestDestroyWindow(DialogPinned.ToSharedRef());
	}

	FString ImportFileContents;
	FFileHelper::LoadFileToString(ImportFileContents, *ImportFilePath);
	if (ImportFileContents.Len() == 0)
	{
		NotifyImportFailed(ImportFilePath, LOCTEXT("EmptyImportFileMessgae", "File was empty."));
		return;
	}

	UMovieSceneComposurePostMoveSettingsSection* PostMoveSettingsSection = CastChecked<UMovieSceneComposurePostMoveSettingsSection>(PostMoveSettingsTrack->CreateNewSection());
	PostMoveSettingsSection->SetRange(TRange<FFrameNumber>::All());

	TArrayView<FMovieSceneFloatChannel*> Channels = PostMoveSettingsSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	FMovieSceneFloatChannel* PivotXCurve        = Channels[0];
	FMovieSceneFloatChannel* PivotYCurve        = Channels[1];
	FMovieSceneFloatChannel* TranslationXCurve  = Channels[2];
	FMovieSceneFloatChannel* TranslationYCurve  = Channels[3];
	FMovieSceneFloatChannel* RotationAngleCurve = Channels[4];
	FMovieSceneFloatChannel* ScaleCurve         = Channels[5];

	TArray<FString> ImportFileLines;
	ImportFileContents.ParseIntoArray(ImportFileLines, TEXT("\n"), true);

	FFrameRate TickResolution = PostMoveSettingsSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	
	FQualifiedFrameTime ImportTime(StartFrame, ImportFrameRate);

	TRange<FFrameNumber> SectionRange(ImportTime.ConvertTo(TickResolution).FloorToFrame());

	int32 LineNumber = 1;
	for (const FString& ImportFileLine : ImportFileLines)
	{
		FString ImportFileLineClean = ImportFileLine.Replace(TEXT("\r"), TEXT(""));

		TArray<FString> ImportFileLineValues;
		ImportFileLineClean.ParseIntoArray(ImportFileLineValues, TEXT(" "), true);

		if (ImportFileLineValues.Num() > 0)
		{
			if (ImportFileLineValues.Num() != 6)
			{
				NotifyImportFailed(ImportFilePath, FText::Format(LOCTEXT("ParseFailedFormat", "Parse failed on line {0}."), LineNumber));
				return;
			}

			const FFrameNumber Time = ImportTime.ConvertTo(TickResolution).FloorToFrame();

			float PivotX = FCString::Atof(*ImportFileLineValues[0]);
			float PivotY = FCString::Atof(*ImportFileLineValues[1]);
			PivotXCurve->AddCubicKey(Time, PivotX);
			PivotYCurve->AddCubicKey(Time, PivotY);

			float TranslationX = FCString::Atof(*ImportFileLineValues[2]);
			float TranslationY = FCString::Atof(*ImportFileLineValues[3]);
			TranslationXCurve->AddCubicKey(Time, TranslationX);
			TranslationYCurve->AddCubicKey(Time, TranslationY);

			float Rotation = FCString::Atof(*ImportFileLineValues[4]);
			RotationAngleCurve->AddCubicKey(Time, Rotation);

			float Scale = FCString::Atof(*ImportFileLineValues[5]);
			ScaleCurve->AddCubicKey(Time, Scale);

			SectionRange.SetUpperBound(TRangeBound<FFrameNumber>::Inclusive(Time));

			++ImportTime.Time.FrameNumber;
		}
		LineNumber++;
	}
	PostMoveSettingsSection->SetRange(SectionRange);

	FScopedTransaction ImportPostMoveSettingsTransaction(NSLOCTEXT("PostMoveSettingsPropertyTrackEditor", "ImportTransaction", "Import post move settings from file"));
	PostMoveSettingsTrack->Modify();
	PostMoveSettingsTrack->RemoveAllAnimationData();
	PostMoveSettingsTrack->AddSection(*PostMoveSettingsSection);
	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}


void FComposurePostMoveSettingsPropertyTrackEditor::ImportCanceled()
{
	TSharedPtr<SWindow> DialogPinned = ImportDialog.Pin();
	if (DialogPinned.IsValid())
	{
		FSlateApplication::Get().RequestDestroyWindow(DialogPinned.ToSharedRef());
	}
}


void FComposurePostMoveSettingsPropertyTrackEditor::GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys)
{
	FPropertyPath StructPath = PropertyChangedParams.StructPathToKey;
	FName ChannelName = StructPath.GetNumProperties() != 0 ? StructPath.GetLeafMostProperty().Property->GetFName() : NAME_None;

	FComposurePostMoveSettings PostMoveSettings = PropertyChangedParams.GetPropertyValue<FComposurePostMoveSettings>();

	const bool bKeyPivot       = ChannelName == NAME_None || ChannelName == GET_MEMBER_NAME_CHECKED(FComposurePostMoveSettings, Pivot);
	const bool bKeyTranslation = ChannelName == NAME_None || ChannelName == GET_MEMBER_NAME_CHECKED(FComposurePostMoveSettings, Translation);
	const bool bKeyRotation    = ChannelName == NAME_None || ChannelName == GET_MEMBER_NAME_CHECKED(FComposurePostMoveSettings, RotationAngle);
	const bool bKeyScale       = ChannelName == NAME_None || ChannelName == GET_MEMBER_NAME_CHECKED(FComposurePostMoveSettings, Scale);

	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(0, PostMoveSettings.Pivot.X,       bKeyPivot));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(1, PostMoveSettings.Pivot.Y,       bKeyPivot));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(2, PostMoveSettings.Translation.X, bKeyTranslation));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel, float>(3, PostMoveSettings.Translation.Y, bKeyTranslation));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(4, PostMoveSettings.RotationAngle, bKeyRotation));
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(5, PostMoveSettings.Scale,         bKeyScale));
}

#undef LOCTEXT_NAMESPACE
