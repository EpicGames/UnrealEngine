// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyTrackEditor.h"
#include "MovieScene/MovieSceneComposurePostMoveSettingsTrack.h"

/** A property track editor for FComposurePostMoveSettings properties. */
class FComposurePostMoveSettingsPropertyTrackEditor
	: public FPropertyTrackEditor<UMovieSceneComposurePostMoveSettingsTrack>
{
public:

	FComposurePostMoveSettingsPropertyTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FPropertyTrackEditor(InSequencer, GetAnimatedPropertyTypes())
	{
	}

	/**
	* Creates an instance of this class.  Called by a sequencer
	*
	* @param OwningSequencer The sequencer instance to be used by this tool
	* @return The new instance of this class
	*/
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

	//~ FMovieSceneTrackEditor interface

	virtual FText GetDisplayName() const override;
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track) override;

	/**
	* Retrieve a list of all property types that this track editor animates
	*/
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromStructType("ComposurePostMoveSettings") });
	}

protected:

	//~ FPropertyTrackEditor interface

	virtual void GenerateKeysFromPropertyChanged(const FPropertyChangedParams& PropertyChangedParams, UMovieSceneSection* SectionToKey, FGeneratedTrackKeys& OutGeneratedKeys) override;

private:

	/** Shows a dialog for importing post move animation from an external file. */
	void ShowImportPostMoveSettingsDialog(UMovieSceneComposurePostMoveSettingsTrack* PostMoveSettingsTrack);

	/** Imports post move settings from an external file to the specified track with the specified settings. */
	void ImportPostMoveSettings(FString ImportFilePath, FFrameRate FrameRate, FFrameNumber StartFrame, UMovieSceneComposurePostMoveSettingsTrack* PostMoveSettingsTrack);
	
	/** Handles closing the import settings dialog when the import it canceled. */
	void ImportCanceled();

private:
	TWeakPtr<SWindow> ImportDialog;
};
