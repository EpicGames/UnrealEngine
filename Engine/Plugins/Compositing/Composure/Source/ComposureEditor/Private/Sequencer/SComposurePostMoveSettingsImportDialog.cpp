// Copyright Epic Games, Inc. All Rights Reserved.

#include "SComposurePostMoveSettingsImportDialog.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SFrameRatePicker.h"
#include "EditorDirectories.h"

#define LOCTEXT_NAMESPACE "PostMoveSettingsImportDialog"

void SComposurePostMoveSettingsImportDialog::Construct(const FArguments& InArgs, FFrameRate InFrameRate, FFrameNumber InStartFrame)
{
	FrameRate  = InFrameRate;
	StartFrame = InStartFrame;
	OnImportSelected = InArgs._OnImportSelected;
	OnImportCanceled = InArgs._OnImportCanceled;

	const FButtonStyle& ButtonStyle = FCoreStyle::Get().GetWidgetStyle< FButtonStyle >("Button");

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("PostMoveSettingsImportDialogTitle", "Import external post moves data"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(350, 170))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				.Padding(10)
				[
					SNew(SGridPanel)
					.FillColumn(1, 0.5f)
					.FillColumn(2, 0.5f)

					// File Path
					+ SGridPanel::Slot(0, 0)
					.Padding(0, 0, 10, 0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FileLabel", "File name"))
					]
					+ SGridPanel::Slot(1, 0)
					.ColumnSpan(2)
					.Padding(0, 0, 0, 0)
					[
						SNew(SFilePathPicker)
						.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
						.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a post moves text file..."))
						.BrowseDirectory(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN))
						.BrowseTitle(LOCTEXT("BrowseButtonTitle", "Choose a post moves text file"))
						.FileTypeFilter(TEXT("Text File (*.txt)|*.txt"))
						.FilePath(this, &SComposurePostMoveSettingsImportDialog::GetFilePath)
						.OnPathPicked(this, &SComposurePostMoveSettingsImportDialog::FilePathPicked)
					]

					// Frame Rate
					+ SGridPanel::Slot(0, 2)
					.Padding(0, 10, 10, 0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FrameRateLabel", "Frame Rate"))
					]
					+ SGridPanel::Slot(1, 2)
					.Padding(0, 10, 0, 0)
					[
						SNew(SFrameRatePicker)
						.Value(this, &SComposurePostMoveSettingsImportDialog::GetFrameRate)
						.OnValueChanged(this, &SComposurePostMoveSettingsImportDialog::FrameRateChanged)
					]

					// Start Frame
					+ SGridPanel::Slot(0, 3)
					.Padding(0, 10, 10, 0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("StartFrameLabel", "Start Frame"))
					]
					+ SGridPanel::Slot(1, 3)
					.Padding(0, 10, 0, 0)
					[
						SNew(SSpinBox<int32>)
						.MinValue(TOptional<int32>())
						.MaxValue(TOptional<int32>())
						.MaxSliderValue(TOptional<int32>())
						.MinSliderValue(TOptional<int32>())
						.Delta(1)
						.Value(this, &SComposurePostMoveSettingsImportDialog::GetStartFrame)
						.OnValueChanged(this, &SComposurePostMoveSettingsImportDialog::StartFrameChanged)
					]
				]
			]

			// Buttons
			+ SVerticalBox::Slot()
			.Padding(10)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			[
				// Import button
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 10, 0)
				.VAlign(VAlign_Bottom)
				[
					SNew(SButton)
					.OnClicked(this, &SComposurePostMoveSettingsImportDialog::OnImportPressed)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ImportButtonLabel", "Import"))
						.Justification(ETextJustify::Center)
						.MinDesiredWidth(90)
					]
				]

				// Cancel button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Bottom)
				[
					SNew(SButton)
					.OnClicked(this, &SComposurePostMoveSettingsImportDialog::OnCancelPressed)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
						.Justification(ETextJustify::Center)
						.MinDesiredWidth(90)
					]
				]
			]
		]);
}

FString SComposurePostMoveSettingsImportDialog::GetFilePath() const
{
	return FilePath;
}

void SComposurePostMoveSettingsImportDialog::FilePathPicked(const FString& PickedPath)
{
	FilePath = PickedPath;
}

FFrameRate SComposurePostMoveSettingsImportDialog::GetFrameRate() const
{
	return FrameRate;
}

int32 SComposurePostMoveSettingsImportDialog::GetStartFrame() const
{
	return StartFrame.Value;
}

void SComposurePostMoveSettingsImportDialog::StartFrameChanged(int32 Value)
{
	StartFrame.Value = Value;
}

void SComposurePostMoveSettingsImportDialog::FrameRateChanged(FFrameRate Value)
{
	FrameRate = Value;
}

FReply SComposurePostMoveSettingsImportDialog::OnCancelPressed()
{
	OnImportCanceled.ExecuteIfBound();
	return FReply::Handled();
}

FReply SComposurePostMoveSettingsImportDialog::OnImportPressed()
{
	OnImportSelected.ExecuteIfBound(FilePath, FrameRate, StartFrame);
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
