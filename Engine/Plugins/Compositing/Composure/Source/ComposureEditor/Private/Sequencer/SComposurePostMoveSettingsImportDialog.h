// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Widgets/SWindow.h"
#include "Misc/FrameRate.h"

/** A dialog for collecting settings for importing post move settings from an external file. */
class SComposurePostMoveSettingsImportDialog : public SWindow
{
public:
	DECLARE_DELEGATE_ThreeParams(FOnImportSelected, FString, FFrameRate, FFrameNumber);
	DECLARE_DELEGATE(FOnImportCanceled);

public:
	SLATE_BEGIN_ARGS(SComposurePostMoveSettingsImportDialog) {}
		/** Event which is executed when the user selects import. */
		SLATE_EVENT(FOnImportSelected, OnImportSelected)
		/** Event which is executed when the user cancels the import. */
		SLATE_EVENT(FOnImportCanceled, OnImportCanceled)
	SLATE_END_ARGS()

	/** 
	 * Constructs a new import dialog. 
	 *
	 * @param InArgs Slate arguments. 
	 * @param InFrameSize The size of the frame in pixels that the import settings are relative to.
	 * @param InFrameRate The framerate of the movie scene which owns the track the data will be imported into. 
	 * @param InStartFrame The target start frame in the movie scene which will be used to import the data.
	 */
	void Construct(const FArguments& InArgs, FFrameRate InFrameRate, FFrameNumber InStartFrame);

private:
	FString GetFilePath() const;
	void FilePathPicked(const FString& PickedPath);

	FFrameRate GetFrameRate() const;
	void FrameRateChanged(FFrameRate Value);

	int32 GetStartFrame() const;
	void StartFrameChanged(int32 Value);

	FReply OnCancelPressed();
	FReply OnImportPressed();

private:
	FString FilePath;
	FFrameRate FrameRate;
	FFrameNumber StartFrame;

	FOnImportSelected OnImportSelected;
	FOnImportCanceled OnImportCanceled;
};