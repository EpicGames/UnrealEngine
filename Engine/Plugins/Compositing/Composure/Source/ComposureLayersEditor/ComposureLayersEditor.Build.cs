// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ComposureLayersEditor : ModuleRules
{
	public ComposureLayersEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"SceneOutliner",
                "WorkspaceMenuStructure",
				"Composure",
				"ClassViewer",
				"DetailCustomizations",
				"PropertyEditor",
				"BlueprintMaterialTextureNodes",
				"AppFramework",
				"LevelEditor",
				"ContentBrowser",
				"MediaIOCore",
				"RHI",
				"RenderCore"
            }
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"EditorFramework",
				"UnrealEd",
			}
		);
	}
}
