// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompElementCollectionViewModel.h"
#include "Engine/Level.h"
#include "Kismet2/SClassPickerDialog.h"
#include "CompositingElement.h"
#include "ClassViewerFilter.h"
#include "EngineAnalytics.h"
#include "IAnalyticsProviderET.h"
#include "LevelEditor.h"
#include "ComposureEditorSettings.h"
#include "Misc/FilterCollection.h"
#include "ScopedTransaction.h"
#include "CompElementViewModel.h"
#include "CompElementEditorCommands.h"
#include "Framework/Commands/GenericCommands.h"
#include "ScopedWorldLevelContext.h"

#define LOCTEXT_NAMESPACE "CompElementsView"

/* CompElementCollectionViewModel_Impl
 *****************************************************************************/

namespace CompElementCollectionViewModel_Impl
{
	static const FName LvlEditorModuleName("LevelEditor");

	/** Opens a modal class picker dialog for selecting the type of element the user wishes to add. */
	static UClass* PromptForElementClass(const FText& PromptTitle, const TArray< TSubclassOf<ACompositingElement> >& ChoiceClasses);
	/** */
	static TArray<FName> GetChildElementNamesRecursive(TWeakObjectPtr<ACompositingElement> RootElement);

	/** Sends analytics of added compositing element */
	static void SendAnalyticsElementAdded(const FString& AddedElementClassName)
	{
		if (!FEngineAnalytics::IsAvailable())
		{
			return;
		}

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ClassName"), AddedElementClassName));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Usage.Composure.AddedElement"), EventAttributes);
	}
}

class FCompElementClassFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet <const UClass*> AllowedChildrenOfClasses;
	TSet <const UClass*> ExcludedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};


static UClass* CompElementCollectionViewModel_Impl::PromptForElementClass(const FText& PromptTitle, const TArray< TSubclassOf<ACompositingElement> >& ChoiceClasses)
{
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	TSharedPtr<FCompElementClassFilter> Filter = MakeShareable(new FCompElementClassFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Options.ExtraPickerCommonClasses.Reserve(ChoiceClasses.Num());
	for (TSubclassOf<ACompositingElement> Class : ChoiceClasses)
	{
		Options.ExtraPickerCommonClasses.Add(Class);
	}

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_HideDropDown;
	Filter->AllowedChildrenOfClasses.Add(ACompositingElement::StaticClass());

	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(PromptTitle, Options, ChosenClass, ACompositingElement::StaticClass());

	return bPressedOk ? ChosenClass : nullptr;
}

static TArray<FName> CompElementCollectionViewModel_Impl::GetChildElementNamesRecursive(TWeakObjectPtr<ACompositingElement> RootElement)
{
	TArray<FName> ElementsToSelect;

	if (RootElement.IsValid())
	{
		for (ACompositingElement* Child : RootElement->GetChildElements())
		{
			if (Child)
			{
				ElementsToSelect.Append(GetChildElementNamesRecursive(Child));
				ElementsToSelect.Add(Child->GetCompElementName());
			}
		}
	}

	return ElementsToSelect;
}

/* FCompElementCollectionViewModel
 *****************************************************************************/

FCompElementCollectionViewModel::FCompElementCollectionViewModel(const TSharedRef<ICompElementManager>& InElementsManager, const TWeakObjectPtr<UEditorEngine>& InEditor)
	: CompElementManager(InElementsManager)
	, Editor(InEditor)
	, CommandList(MakeShareable(new FUICommandList))
	, Filters(MakeShareable(new FCompElementFilterCollection))	
{}

FCompElementCollectionViewModel::~FCompElementCollectionViewModel()
{
	if (Editor.IsValid())
	{
		Editor->UnregisterForUndo(this);
	}

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(CompElementCollectionViewModel_Impl::LvlEditorModuleName);
	LevelEditor.OnActorSelectionChanged().RemoveAll(this);

	Filters->OnChanged().RemoveAll(this);
	CompElementManager->OnElementsChanged().RemoveAll(this);
}

void FCompElementCollectionViewModel::Initialize()
{
	BindCommands();

	CompElementManager->OnElementsChanged().AddSP(this, &FCompElementCollectionViewModel::OnElementsChanged);
	Filters->OnChanged().AddSP(this, &FCompElementCollectionViewModel::OnFilterChanged);

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(CompElementCollectionViewModel_Impl::LvlEditorModuleName);
	// Tell the level editor we want to be notified when selection changes
	LevelEditor.OnActorSelectionChanged().AddRaw(this, &FCompElementCollectionViewModel::OnActorSelectionChanged);

	if (Editor.IsValid())
	{
		Editor->RegisterForUndo(this);
	}

	Refresh(); 
}

void FCompElementCollectionViewModel::AddFilter(const TSharedRef<FCompElementFilter>& InFilter)
{
	Filters->Add(InFilter);
	OnFilterChanged();
}

void FCompElementCollectionViewModel::RemoveFilter(const TSharedRef<FCompElementFilter>& InFilter)
{
	Filters->Remove(InFilter);
	OnFilterChanged();
}

TArray< TSharedPtr<FCompElementViewModel> >& FCompElementCollectionViewModel::GetRootCompElements()
{
	return FilteredRootItems;
}

void FCompElementCollectionViewModel::GetChildElements(TSharedPtr<FCompElementViewModel> ParentPtr, TArray< TSharedPtr<FCompElementViewModel> >& OutChildElements)
{
	FilteredChildren.MultiFind(ParentPtr, OutChildElements, /*bMaintainOrder =*/true);
}

TSharedPtr<FCompElementViewModel> FCompElementCollectionViewModel::GetSelectionProxy(const TSharedPtr<FCompElementViewModel>& SelectedItem) const
{
	if (SelectedItem->IsEditable())
	{
		return SelectedItem;
	}

	struct GetSelectionProxy_Impl
	{
		static TSharedPtr<FCompElementViewModel> FindEditableParent(const TSharedPtr<FCompElementViewModel>& Target, const TArray< TSharedPtr<FCompElementViewModel> >& SearchList)
		{
			for (const TSharedPtr<FCompElementViewModel>& Element : SearchList)
			{
				if (Element == Target)
				{
					return Element;
				}
				else
				{
					TSharedPtr<FCompElementViewModel> SearchResult = FindEditableParent(Target, Element->Children);
					if (SearchResult.IsValid())
					{
						if (!SearchResult->IsEditable())
						{
							return Element;
						}
						return SearchResult;
					}
				}
			}
			return nullptr;
		}
	};

	return GetSelectionProxy_Impl::FindEditableParent(SelectedItem, RootViewModels);
}

const TArray< TSharedPtr<FCompElementViewModel> >& FCompElementCollectionViewModel::GetSelectedElements() const
{
	return SelectedElements;
}

void FCompElementCollectionViewModel::GetSelectedElementNames(OUT TArray<FName>& OutSelectedElementNames) const
{
	AppendSelectedElementNames(OutSelectedElementNames);
}

void FCompElementCollectionViewModel::SetSelectedElements(const TArray< TSharedPtr<FCompElementViewModel> >& InSelectedElements)
{
	SelectedElements.Empty();
	SelectedElements.Append(InSelectedElements);

	RefreshActorSelections();
	SelectionChanged.Broadcast();
}

void FCompElementCollectionViewModel::SetSelectedElement(const FName& ElementName)
{
	SelectedElements.Empty();

	for (TSharedPtr<FCompElementViewModel> ViewModel : FilteredRootItems)
	{
		if (ElementName == ViewModel->GetFName())
		{
			SelectedElements.Add(ViewModel);
			break;
		}

		TArray< TSharedPtr<FCompElementViewModel> > ChildElements;
		FilteredChildren.MultiFind(ViewModel, ChildElements);

		for (TSharedPtr<FCompElementViewModel> Element : ChildElements)
		{
			if (ElementName == Element->GetFName())
			{
				SelectedElements.Add(Element);
			}
		}
	}

	RefreshActorSelections();
	SelectionChanged.Broadcast();
}

const TSharedRef<FUICommandList> FCompElementCollectionViewModel::GetCommandList() const
{
	return CommandList;
}

void FCompElementCollectionViewModel::BindCommands()
{
	const FCompElementEditorCommands& Commands = FCompElementEditorCommands::Get();
	FUICommandList& ActionList = *CommandList;

	ActionList.MapAction(Commands.CreateEmptyComp,
		FExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::CreateTopLevelElement_Executed),
		FCanExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::CreateTopLevelElement_CanExecute));

	ActionList.MapAction(Commands.CreateNewElement,
		FExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::CreateChildElement_Executed),
		FCanExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::CreateChildElement_CanExecute));

	ActionList.MapAction(Commands.RefreshCompList,
		FExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::RefreshList_Executed),
		FCanExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::RefreshList_CanExecute));

	ActionList.MapAction(Commands.OpenElementPreview,
		FExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::OpenPreview_Executed),
		FCanExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::OpenPreview_CanExecute));

	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	ActionList.MapAction(GenericCommands.Cut,
		FExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::CutElements_Executed),
		FCanExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::CutElements_CanExecute));

	ActionList.MapAction(GenericCommands.Copy,
		FExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::CopyElements_Executed),
		FCanExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::CopyElements_CanExecute));

	ActionList.MapAction(GenericCommands.Paste,
		FExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::PasteElements_Executed),
		FCanExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::PasteElements_CanExecute));

	ActionList.MapAction(GenericCommands.Duplicate,
		FExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::DuplicateElements_Executed),
		FCanExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::DuplicateElements_CanExecute));

	ActionList.MapAction(GenericCommands.Delete,
		FExecuteAction::CreateSP( this, &FCompElementCollectionViewModel::DeleteElements_Executed ),
		FCanExecuteAction::CreateSP( this, &FCompElementCollectionViewModel::DeleteElements_CanExecute ) );

	ActionList.MapAction(GenericCommands.Rename,
		FExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::RequestRenameElement_Executed),
		FCanExecuteAction::CreateSP(this, &FCompElementCollectionViewModel::RequestRenameElement_CanExecute));
}

void FCompElementCollectionViewModel::Refresh()
{
	CompElementManager->RefreshElementsList();
}

void FCompElementCollectionViewModel::OnFilterChanged()
{
	RefreshFilteredElements();
	ElementsChanged.Broadcast(ECompElementEdActions::Reset, nullptr, NAME_None);
}

void FCompElementCollectionViewModel::OnElementsChanged(const ECompElementEdActions Action, const TWeakObjectPtr<ACompositingElement>& ChangedComp, const FName& ChangedProperty)
{
	switch (Action)
	{
	case ECompElementEdActions::Add:
		OnElementAdded(ChangedComp);
		break;

	case ECompElementEdActions::Rename:
		// We purposely ignore re-filtering in this case
		SortFilteredElements();
		break;

	case ECompElementEdActions::Modify:
		RefreshFilteredElements();
		break;

	case ECompElementEdActions::Delete:
		OnElementDelete();
		break;

	case ECompElementEdActions::Reset:
	default:
		//OnResetElements();
		break;
	}
	OnResetElements();
	ElementsChanged.Broadcast(Action, ChangedComp, ChangedProperty);
}

void FCompElementCollectionViewModel::OnElementAdded(const TWeakObjectPtr<ACompositingElement>& AddedElementPtr)
{
	if (!AddedElementPtr.IsValid())
	{
		OnResetElements();
		return;
	}
	ACompositingElement* AddedElementObj = AddedElementPtr.Get();

	TSharedPtr<FCompElementViewModel> ParentPtr;
	if (AddedElementObj->IsSubElement() && !TryGetViewModel(AddedElementObj->GetElementParent(), ParentPtr))
	{
		OnResetElements();
		return;
	}
	
	CompElementCollectionViewModel_Impl::SendAnalyticsElementAdded(AddedElementObj->GetClass()->GetName());

	const TSharedRef<FCompElementViewModel> NewElementModel = FCompElementViewModel::Create(AddedElementObj, CompElementManager);
	if (ParentPtr)
	{
		ParentPtr->Children.Add(NewElementModel);

		ensureMsgf(FilteredRootItems.Remove(NewElementModel) == 0, TEXT("Catching an issue when the comp elements list already contains an entry for this item - please notify the dev team with a repro."));

		TArray< TSharedPtr<FCompElementViewModel> > ExistingChildren;
		FilteredChildren.GenerateValueArray(ExistingChildren);

		// We specifically ignore filters when dealing with single additions
		if (ensureMsgf(!ExistingChildren.Contains(NewElementModel), TEXT("Catching an issue when the comp elements list already contains an entry for this item - please notify the dev team with a repro.")))
		{
			FilteredChildren.Add(ParentPtr, NewElementModel);
		}
	}
	else
	{
		RootViewModels.Add(NewElementModel);
		// We specifically ignore filters when dealing with single additions
		if (ensureMsgf(!FilteredRootItems.Contains(NewElementModel), TEXT("Catching an issue when the comp elements list already contains an entry for this item - please notify the dev team with a repro.")))
		{
			FilteredRootItems.Add(NewElementModel);
		}
	}

	SortFilteredElements();
}

void FCompElementCollectionViewModel::OnElementDelete()
{
	TArray< TWeakObjectPtr<ACompositingElement> > AuthoratativeElementList;
	CompElementManager->AddAllCompElementsTo(AuthoratativeElementList);

	DestructivelyPurgeInvalidViewModels(AuthoratativeElementList);
}

void FCompElementCollectionViewModel::OnElementAttached(const TWeakObjectPtr<ACompositingElement>& AttachedElement)
{
	if (!AttachedElement.IsValid() || !AttachedElement->IsSubElement())
	{
		OnResetElements();
		return;
	}

	TSharedPtr<FCompElementViewModel> ElementModel;
	TSharedPtr<FCompElementViewModel> ParentModel;

	if (!TryGetViewModel(AttachedElement, ElementModel) || !TryGetViewModel(AttachedElement->GetElementParent(), ParentModel))
	{
		OnResetElements();
		return;
	}

	RootViewModels.Remove(ElementModel);
	ParentModel->Children.Add(ElementModel);

	RefreshFilteredElements();
}

void FCompElementCollectionViewModel::OnResetElements()
{
	TArray< TWeakObjectPtr<ACompositingElement> > AuthoratativeElementList;
	// Expected: AuthoratativeElementList doesn't contain invalid entries
	CompElementManager->AddAllCompElementsTo(AuthoratativeElementList);

	FilteredRootItems.Empty();
	FilteredChildren.Empty();

	// Purge any invalid view-models, 
	// This function also removes any elements already with view-model representations from AuthoratativeElementList
	DestructivelyPurgeInvalidViewModels(AuthoratativeElementList);

	RebuildViewModelHierarchy();

	// Create any missing view-models
	CreateViewModels(AuthoratativeElementList);

	// Rebuild the filtered elements list
	RefreshFilteredElements();
}

void FCompElementCollectionViewModel::DestructivelyPurgeInvalidViewModels(TArray< TWeakObjectPtr<ACompositingElement> >& InElements)
{
	struct PurgeInvalidViewModels_Impl
	{
		typedef TFunction<void(const TSharedPtr<FCompElementViewModel>&, const TSharedPtr<FCompElementViewModel>&)> FOnRemovalCallback;

		void RemoveInvalidViewModels(TArray< TSharedPtr<FCompElementViewModel> >& ViewModelList, FOnRemovalCallback& OnRemoval, TSharedPtr<FCompElementViewModel> Parent = nullptr)
		{
			for (int32 ElementIndex = ViewModelList.Num() - 1; ElementIndex >= 0; --ElementIndex)
			{
				const TSharedPtr<FCompElementViewModel> ElementViewModel = ViewModelList[ElementIndex];
				TWeakObjectPtr<ACompositingElement> ElementObj = ElementViewModel->GetDataSource();

				if (!ElementObj.IsValid() || ElementsSrcList.Remove(ElementObj) == 0)
				{
					ViewModelList.RemoveAtSwap(ElementIndex);
					OnRemoval(ElementViewModel, Parent);
				}
				else
				{
					RemoveInvalidViewModels(ElementViewModel->Children, OnRemoval, ElementViewModel);
				}
			}
		}

		PurgeInvalidViewModels_Impl(TArray< TWeakObjectPtr<ACompositingElement> >& InElementsSrcList)
			: ElementsSrcList(InElementsSrcList)
		{}

		TArray< TWeakObjectPtr<ACompositingElement> >& ElementsSrcList;
	};

	PurgeInvalidViewModels_Impl::FOnRemovalCallback OnInvalidViewModelFound([this](const TSharedPtr<FCompElementViewModel>& InvalidViewModel, const TSharedPtr<FCompElementViewModel>& Parent)
	{
		SelectedElements.Remove(InvalidViewModel);

		if (!Parent.IsValid())
		{
			FilteredRootItems.Remove(InvalidViewModel);
			FilteredChildren.Remove(InvalidViewModel);
		}
		else
		{
			FilteredChildren.RemoveSingle(Parent, InvalidViewModel);
		}
	});

	PurgeInvalidViewModels_Impl(InElements).RemoveInvalidViewModels(RootViewModels, OnInvalidViewModelFound);
}

void FCompElementCollectionViewModel::CreateViewModels(TArray< TWeakObjectPtr<ACompositingElement> >& InElements)
{
	struct CreateViewModels_Impl
	{
		const TSharedRef<FCompElementViewModel> CreateViewModel(const TWeakObjectPtr<ACompositingElement>& ElementPtr)
		{
			const TSharedRef<FCompElementViewModel> NewViewModel = FCompElementViewModel::Create(ElementPtr, This->CompElementManager);
			//ViewModelRefTable.Add(ElementPtr, NewViewModel);

			if (!ElementPtr.IsValid())
			{
				UE_LOGF(LogTemp, Warning, "FCompElementCollectionViewModel::CreateViewModels - Invalid element");
				return NewViewModel;
			}

			if (ElementPtr->IsSubElement())
			{
				TSharedPtr<FCompElementViewModel> ParentViewModel;

				TWeakObjectPtr<ACompositingElement> ParentObj = ElementPtr->GetElementParent();
				const int32 ParentIndex = Elements.Find(ParentObj);

				if (ParentIndex != INDEX_NONE && Visited[ParentIndex] == false)
				{
					Visited[ParentIndex] = true;
					ParentViewModel = CreateViewModel(Elements[ParentIndex]);
				}
				else
				{
					This->TryGetViewModel(ParentObj, ParentViewModel);
				}

				if (ParentViewModel.IsValid())
				{
					ParentViewModel->Children.Add(NewViewModel);
				}
				else
				{
					UE_LOGF(LogTemp, Warning, "FCompElementCollectionViewModel::CreateViewModels - Invalid parent view model %d", ParentIndex);

					// fallback to adding it to RootViews
					This->RootViewModels.Add(NewViewModel);
				}
			}
			else
			{
				This->RootViewModels.Add(NewViewModel);
			}

			return NewViewModel;
		}

		CreateViewModels_Impl(FCompElementCollectionViewModel* InThis, TArray< TWeakObjectPtr<ACompositingElement> >& ElementList)
			: This(InThis)
			, Elements(ElementList)
		{
			Visited.SetNumZeroed(Elements.Num());
		}

		FCompElementCollectionViewModel* This;
		TArray< TWeakObjectPtr<ACompositingElement> >& Elements;
		TArray<bool> Visited;
	};
	CreateViewModels_Impl CreateHelper(this, InElements);

	for (int32 i = 0; i < InElements.Num(); ++i)
	{
		if (CreateHelper.Visited[i] == false)
		{
			CreateHelper.Visited[i] = true;
			CreateHelper.CreateViewModel(InElements[i]);
		}
	}
}


void FCompElementCollectionViewModel::RebuildViewModelHierarchy()
{
	TArray< TSharedPtr<FCompElementViewModel> > AllViewModels;
	GetAllViewModels(AllViewModels);

	// can't rely on TryGetViewModel() since it relies on walking the hierarchy which we are
	// in the midst of reforming
	auto FindViewModel = [&AllViewModels](ACompositingElement* ElementObj)->TSharedPtr<FCompElementViewModel>
	{
		for (const TSharedPtr<FCompElementViewModel>& ViewModel : AllViewModels)
		{
			if (ElementObj == ViewModel->GetDataSource())
			{
				return ViewModel;
			}
		}
		return nullptr;
	};

	RootViewModels.Empty(RootViewModels.Num());

	for (const TSharedPtr<FCompElementViewModel>& ViewModel : AllViewModels)
	{
		TWeakObjectPtr<ACompositingElement> ElementPtr = ViewModel->GetDataSource();
		if (ElementPtr.IsValid())
		{
			ACompositingElement* ElementObj = ElementPtr.Get();
			if (!ElementObj->IsSubElement())
			{
				RootViewModels.Add(ViewModel);
			}

			const TArray<ACompositingElement*> Children = ElementObj->GetChildElements();
			ViewModel->Children.Empty(Children.Num());

			for (ACompositingElement* Child : Children)
			{
				TSharedPtr<FCompElementViewModel> ChildViewModel = FindViewModel(Child);
				if (ChildViewModel.IsValid())
				{
					ViewModel->Children.Add(ChildViewModel);
				}
			}
		}
		else
		{
			ViewModel->Children.Empty();
		}
	}
}

void FCompElementCollectionViewModel::RefreshFilteredElements()
{
	FilteredRootItems.Empty();
	FilteredChildren.Empty();

	struct RefreshFilteredElements_Impl
	{
		bool FilterChildren(const TSharedPtr<FCompElementViewModel>& ViewModel, FFilteredChildList& OutFilteredChildren)
		{
			bool bChildIncluded = false;
			for (const TSharedPtr<FCompElementViewModel>& Child : ViewModel->Children)
			{
				if (FilterChildren(Child, OutFilteredChildren) || FilterRef->PassesAllFilters(Child))
				{
					TArray< TSharedPtr<FCompElementViewModel> > ExistingChildren;
					OutFilteredChildren.GenerateValueArray(ExistingChildren);

					if (ensureMsgf(!ExistingChildren.Contains(Child), TEXT("Catching an issue when the comp elements list already contains an entry for this item - please notify the dev team with a repro.")))
					{
						OutFilteredChildren.Add(ViewModel, Child);
					}
					bChildIncluded = true;
				}
			}

			return bChildIncluded;
		}

		RefreshFilteredElements_Impl(const TSharedRef<FCompElementFilterCollection>& InFilterRef)
			: FilterRef(InFilterRef)
		{}

		const TSharedRef<FCompElementFilterCollection> FilterRef;
	};
	RefreshFilteredElements_Impl FilterHelper(Filters);


	for (const TSharedPtr<FCompElementViewModel>& ViewModel : RootViewModels)
	{
		if (FilterHelper.FilterChildren(ViewModel, FilteredChildren) || Filters->PassesAllFilters(ViewModel))
		{
			if (ensureMsgf(!FilteredRootItems.Contains(ViewModel), TEXT("Catching an issue when the comp elements list already contains an entry for this item - please notify the dev team with a repro.")))
			{
				FilteredRootItems.Add(ViewModel);
			}
		}
	}

	SortFilteredElements();
}

void FCompElementCollectionViewModel::SortFilteredElements()
{
	struct FCompareRootElements
	{
		FORCEINLINE bool operator()(const TSharedPtr<FCompElementViewModel>& Lhs, const TSharedPtr<FCompElementViewModel>& Rhs) const
		{
			return Lhs->GetFName().Compare(Rhs->GetFName()) < 0;
		}
	};
	FilteredRootItems.Sort(FCompareRootElements());

	struct FCompareChildElements
	{
		FORCEINLINE bool operator()(const TSharedPtr<FCompElementViewModel>& Lhs, const TSharedPtr<FCompElementViewModel>& Rhs) const
		{
			TWeakObjectPtr<ACompositingElement> LhsDataSrc = Lhs->GetDataSource();
			TWeakObjectPtr<ACompositingElement> RhsDataSrc = Rhs->GetDataSource();

			if (!LhsDataSrc.IsValid() || !RhsDataSrc.IsValid())
			{
				return LhsDataSrc.IsValid();
			}
			else
			{
				ACompositingElement* LhsParent = LhsDataSrc->GetElementParent();
				ACompositingElement* RhsParent = RhsDataSrc->GetElementParent();

				if (LhsParent && LhsParent == RhsParent)
				{
					const TArray<ACompositingElement*> Children = LhsParent->GetChildElements();
					const int32 LhsIndex = Children.Find(LhsDataSrc.Get());
					const int32 RhsIndex = Children.Find(RhsDataSrc.Get());

					return LhsIndex < RhsIndex;
				}
				else if (LhsParent && RhsParent)
				{
					return LhsParent->GetCompElementName().Compare(RhsParent->GetCompElementName()) < 0;
				}
				else if (LhsParent != RhsParent)
				{
					return LhsParent != nullptr;
				}
			}

			return Lhs->GetFName().Compare(Rhs->GetFName()) < 0;
		}
	};
	FilteredChildren.ValueSort(FCompareChildElements());
}



TSharedPtr<FCompElementViewModel> FCompElementCollectionViewModel::GetViewModel(TWeakObjectPtr<ACompositingElement> CompObjPtr)
{
	struct GetViewModel_Impl
	{
		TSharedPtr<FCompElementViewModel> RecursiveSearch(const TArray< TSharedPtr<FCompElementViewModel> >& RootModels)
		{
			TSharedPtr<FCompElementViewModel> FoundViewModel;
			for (const TSharedPtr<FCompElementViewModel>& ViewModel : RootModels)
			{
				if (ViewModel.IsValid())
				{
					if (ViewModel->GetDataSource() == DataSrcToMatch)
					{
						FoundViewModel = ViewModel;
					}
					else
					{
						FoundViewModel = RecursiveSearch(ViewModel->Children);
					}
				}

				if (FoundViewModel.IsValid())
				{
					break;
				}
			}
			return FoundViewModel;
		}

		GetViewModel_Impl(TWeakObjectPtr<ACompositingElement> InCompObjPtr)
			: DataSrcToMatch(InCompObjPtr)
		{}

		TWeakObjectPtr<ACompositingElement> DataSrcToMatch;
	};

	return GetViewModel_Impl(CompObjPtr).RecursiveSearch(RootViewModels);
}

bool FCompElementCollectionViewModel::TryGetViewModel(TWeakObjectPtr<ACompositingElement> CompObjPtr, TSharedPtr<FCompElementViewModel>& OutViewModel)
{
	OutViewModel = GetViewModel(CompObjPtr);
	return OutViewModel.IsValid();
}

void FCompElementCollectionViewModel::GetAllViewModels(TArray< TSharedPtr<FCompElementViewModel> >& OutAllViewModels)
{
	struct GetAllViewModels_Impl
	{
		void RecursiveAppend(const TArray< TSharedPtr<FCompElementViewModel> >& RootModels)
		{
			ViewModelsOut.Append(RootModels);

			for (const TSharedPtr<FCompElementViewModel>& ViewModel : RootModels)
			{
				RecursiveAppend(ViewModel->Children);
			}
		}

		GetAllViewModels_Impl(TArray< TSharedPtr<FCompElementViewModel> >& InViewModelsOut)
			: ViewModelsOut(InViewModelsOut)
		{}

		TArray< TSharedPtr<FCompElementViewModel> >& ViewModelsOut;
	};

	GetAllViewModels_Impl(OutAllViewModels).RecursiveAppend(RootViewModels);
}

void FCompElementCollectionViewModel::AppendSelectedElementNames(TArray<FName>& OutElementNames) const
{
	for (TSharedPtr<FCompElementViewModel> SelectedItem : SelectedElements)
	{
		OutElementNames.Add(SelectedItem->GetFName());
	}
}

void FCompElementCollectionViewModel::OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	SelectedElements.Empty();
	for (UObject* SelectedObj : NewSelection)
	{
		ACompositingElement* CompActor = Cast<ACompositingElement>(SelectedObj);

		TSharedPtr<FCompElementViewModel> FoundViewModel;
		if (TryGetViewModel(CompActor, FoundViewModel))
		{
			SelectedElements.Add(FoundViewModel);
		}
		else
		{
			SelectedElements.Empty();
			break;
		}
	}
	SelectionChanged.Broadcast();
}

void FCompElementCollectionViewModel::RefreshActorSelections() const
{
	TArray<FName> SelectedElementNames;
	AppendSelectedElementNames(SelectedElementNames);

	Editor->SelectNone(/*bNotifySelectNone =*/SelectedElementNames.Num() == 0, /*bDeselectBSPSurfs =*/true);
	CompElementManager->SelectElementActors(SelectedElementNames, /*bSelectActors =*/true, /*bNotifySelectActors =*/true, /*bSelectEvenIfHidden =*/true);
}

void FCompElementCollectionViewModel::CreateTopLevelElement_Executed()
{
	TArray< TSubclassOf<ACompositingElement> > HighlightedClasses;

	const UComposureEditorSettings* CompEditorSettings = GetDefault<UComposureEditorSettings>();
	for (FSoftObjectPath FeaturedClass : CompEditorSettings->GetFeaturedCompShotClasses())
	{
		TSubclassOf<ACompositingElement> ClassObj = Cast<UClass>(FeaturedClass.TryLoad());
		if (ClassObj != nullptr)
		{
			HighlightedClasses.Add(ClassObj);
		}
	}

	if (HighlightedClasses.Num() == 0)
	{
		HighlightedClasses.Add(ACompositingElement::StaticClass());
	}

	UClass* ChosenClass = CompElementCollectionViewModel_Impl::PromptForElementClass(LOCTEXT("PickCompClass", "Pick an Comp Class"), HighlightedClasses);

	if (ChosenClass)
	{
		const FScopedTransaction Transaction(LOCTEXT("CreateEmptyComp", "Create Comp"));
		const FName NewCompName = GenerateUniqueCompName();
		CompElementManager->CreateElement(NewCompName, ChosenClass);

		SetSelectedElement(NewCompName);

		if (RequestRenameElement_CanExecute())
		{
			RequestRenameElement_Executed();
		}
	}
}

FName FCompElementCollectionViewModel::GenerateUniqueCompName() const
{
	FName ShotName;

	int32 CompIndex = 0;
	TWeakObjectPtr<ACompositingElement> ExistingComp;
	do
	{
		++CompIndex;
		ShotName = FName(*FString::Printf(TEXT("%03d0_comp"), CompIndex));

	} while (CompElementManager->TryGetElement(ShotName, ExistingComp));

	return ShotName;
}

bool FCompElementCollectionViewModel::CreateTopLevelElement_CanExecute() const
{
	return true;
}

void FCompElementCollectionViewModel::CreateChildElement_Executed()
{
	const UComposureEditorSettings* CompEditorSettings = GetDefault<UComposureEditorSettings>();

	TArray< TSubclassOf<ACompositingElement> > HighlightedClasses;
	for (FSoftObjectPath FeaturedClass : CompEditorSettings->GetFeaturedElementClasses())
	{
		TSubclassOf<ACompositingElement> ClassObj = Cast<UClass>(FeaturedClass.TryLoad());
		if (ClassObj != nullptr)
		{
			HighlightedClasses.Add(ClassObj);
		}
	}

	UClass* ChosenClass = CompElementCollectionViewModel_Impl::PromptForElementClass(LOCTEXT("PickElementClass", "Pick an Element Type"), HighlightedClasses);

	if (ChosenClass)
	{
		TSharedPtr<FCompElementViewModel> SelectedParent;
		AActor* LevelContext = nullptr;

		if (SelectedElements.Num() > 0)
		{
			SelectedParent = SelectedElements[0];

			TWeakObjectPtr<ACompositingElement> ParentObj = SelectedParent->GetDataSource();
			if (ParentObj.IsValid())
			{
				LevelContext = ParentObj.Get();
			}
		}

		const FScopedTransaction Transaction(LOCTEXT("CreateNewElement", "New Comp Element"));
		const FName NewCompName = GenerateUniqueElementName(ChosenClass);
		CompElementManager->CreateElement(NewCompName, ChosenClass, LevelContext);

		if (SelectedParent.IsValid())
		{
			CompElementManager->AttachCompElement(SelectedParent->GetFName(), NewCompName);
		}

		SetSelectedElement(NewCompName);

		if (RequestRenameElement_CanExecute())
		{
			RequestRenameElement_Executed();
		}
	}
}

FName FCompElementCollectionViewModel::GenerateUniqueElementName(TSubclassOf<ACompositingElement> ElementClass) const
{
	FName ElementName;
	FString BaseName = LOCTEXT("DefaultElementName", "layer_element").ToString();

	const UDefaultComposureEditorSettings* CompEdSettings = GetDefault<UDefaultComposureEditorSettings>();
	if (ElementClass)
	{
		const FString* DefaultNamePtr = CompEdSettings->DefaultElementNames.Find(ElementClass->GetFName());
		if (DefaultNamePtr)
		{
			BaseName = *DefaultNamePtr;
		}
	}

	int32 ElementIndex = 0;
	TWeakObjectPtr<ACompositingElement> ExistingElement;
	do
	{
		++ElementIndex;
		ElementName = FName(*FString::Printf(TEXT("%s%d"), *BaseName, ElementIndex));

	} while (CompElementManager->TryGetElement(ElementName, ExistingElement));

	return ElementName;
}

bool FCompElementCollectionViewModel::CreateChildElement_CanExecute() const
{
	return SelectedElements.Num() == 1 && SelectedElements[0]->GetDataSource().IsValid();
}

void FCompElementCollectionViewModel::CutElements_Executed()
{
	CopyElements_Executed();
	DeleteElements_Executed();
}

bool FCompElementCollectionViewModel::CutElements_CanExecute() const
{
	return CopyElements_CanExecute();
}

void FCompElementCollectionViewModel::CopyElements_Executed()
{
	TArray<FName> CachedElementNames;
	
	TArray<FName> ChildElementsToCopy;
	for (TSharedPtr<FCompElementViewModel> Element : SelectedElements)
	{
		CachedElementNames.Add(Element->GetFName());
		ChildElementsToCopy.Append(CompElementCollectionViewModel_Impl::GetChildElementNamesRecursive(Element->GetDataSource()));
	}

	CompElementManager->SelectElementActors(ChildElementsToCopy, /*bSelectActors =*/true, /*bNotifySelectActors =*/true, /*bSelectEvenIfHidden =*/true);

	Editor->CopySelectedActorsToClipboard(GEditor->GetEditorWorldContext().World(), false);

	Editor->SelectNone(/*bNotifySelectNone =*/false, /*bDeselectBSPSurfs =*/true);
	CompElementManager->SelectElementActors(CachedElementNames, /*bSelectActors =*/true, /*bNotifySelectActors =*/true, /*bSelectEvenIfHidden =*/true);
}

bool FCompElementCollectionViewModel::CopyElements_CanExecute() const
{
	return SelectedElements.Num() > 0;
}

void FCompElementCollectionViewModel::PasteElements_Executed()
{
	TSharedPtr<FCompElementViewModel> PrevSelection;
	ULevel* LevelContext = nullptr;
	TWeakObjectPtr<ACompositingElement> PrevSelectionObj;

	if (SelectedElements.Num() > 0)
	{
		PrevSelection = SelectedElements[0];

		PrevSelectionObj = PrevSelection->GetDataSource();
		if (PrevSelectionObj.IsValid())
		{
			LevelContext = PrevSelectionObj->GetLevel();
		}
	}
	if (LevelContext == nullptr)
	{
		for (TSharedPtr<FCompElementViewModel> Model : RootViewModels)
		{
			TWeakObjectPtr<ACompositingElement> ModelObj = Model->GetDataSource();
			if (ModelObj.IsValid())
			{
				LevelContext = ModelObj->GetLevel();
				if (LevelContext)
				{
					break;
				}
			}
		}
	}

	UWorld* TargetWorld = GEditor->GetEditorWorldContext().World();
	if (LevelContext)
	{
		TargetWorld = LevelContext->GetWorld();
	}
	else
	{
		LevelContext = TargetWorld->GetCurrentLevel();
	}

	{
		FScopedWorldLevelContext ScopedLevelContext(TargetWorld, LevelContext);
		Editor->PasteSelectedActorsFromClipboard(TargetWorld, FText::FromString("Comp Element Paste"), EPasteTo::PT_Here);
	}

	if (SelectedElements.Num() > 0)
	{
		TSharedPtr<FCompElementViewModel> NewPastedElement = SelectedElements[0];

		if (PrevSelectionObj.IsValid())
		{
			if (ACompositingElement* PrevSelectionParent = PrevSelectionObj->GetElementParent())
			{
				// Selected element has a parent, make it a sibling
				CompElementManager->AttachCompElement(PrevSelectionParent->GetFName(), NewPastedElement->GetFName());
			}
			else
			{
				// Selected element is root, make pasted element a child
				CompElementManager->AttachCompElement(PrevSelection->GetFName(), NewPastedElement->GetFName());
			}
		}
		else if (ACompositingElement* Parent = NewPastedElement->GetDataSource()->GetElementParent())
		{
			Parent->DetatchAsChildLayer(NewPastedElement->GetDataSource().Get());
		}
	}

	Refresh();
}

bool FCompElementCollectionViewModel::PasteElements_CanExecute() const
{
	// Currently allowing anything to be pasted, but it may be worth revisiting this to filter out non-related actors
	return true;
}

void FCompElementCollectionViewModel::DuplicateElements_Executed()
{
	CopyElements_Executed();
	PasteElements_Executed();
}

bool FCompElementCollectionViewModel::DuplicateElements_CanExecute() const
{
	return CopyElements_CanExecute();
}

void FCompElementCollectionViewModel::DeleteElements_Executed()
{
	if( SelectedElements.Num() == 0 )
	{
		return;
	}

	TArray<FName> SelectedElementNames;
	for (TSharedPtr<FCompElementViewModel> Element : SelectedElements)
	{
		SelectedElementNames.Append(CompElementCollectionViewModel_Impl::GetChildElementNamesRecursive(Element->GetDataSource()));
	}
	const FScopedTransaction Transaction(LOCTEXT("DeleteComp", "Delete Comp Elements"));

	CompElementManager->SelectElementActors(SelectedElementNames, /*bSelectActors =*/true, /*bNotifySelectActors =*/true, /*bSelectEvenIfHidden =*/true);

	TArray<FName> ElementsToDelete;
	AppendSelectedElementNames(ElementsToDelete);
	CompElementManager->DeleteElements(ElementsToDelete);
}

bool FCompElementCollectionViewModel::DeleteElements_CanExecute() const
{
	return SelectedElements.Num() > 0;
}

void FCompElementCollectionViewModel::RequestRenameElement_Executed()
{
	if (SelectedElements.Num() == 1)
	{
		OnRenameRequested().Broadcast();
	}
}

bool FCompElementCollectionViewModel::RequestRenameElement_CanExecute() const
{
	return SelectedElements.Num() == 1 && SelectedElements[0]->IsEditable();
}

void FCompElementCollectionViewModel::OpenPreview_Executed()
{
	if (SelectedElements.Num() > 0)
	{
		SelectedElements[0]->BroadcastPreviewRequest();
	}
}

bool FCompElementCollectionViewModel::OpenPreview_CanExecute() const
{
	return SelectedElements.Num() == 1 && SelectedElements[0]->GetDataSource().IsValid();
}

void FCompElementCollectionViewModel::RefreshList_Executed()
{
	Refresh();
}

bool FCompElementCollectionViewModel::RefreshList_CanExecute() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
