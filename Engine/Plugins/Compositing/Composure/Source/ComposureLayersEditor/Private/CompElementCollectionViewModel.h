// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EditorUndoClient.h"
#include "ICompElementManager.h" // for ICompElementManager::FOnElementsChanged
#include "UObject/WeakObjectPtr.h"

class UEditorEngine;
class FCompElementViewModel;
class UClass;
class FUICommandList;

template<typename TItemType> class IFilter;
typedef IFilter<const TSharedPtr<FCompElementViewModel>&> FCompElementFilter;

template<typename ItemType> class TFilterCollection;
typedef TFilterCollection<const TSharedPtr<FCompElementViewModel>&> FCompElementFilterCollection;

/**
 * The non-UI solution specific presentation logic for a comp elements' view.
 */
class FCompElementCollectionViewModel : public TSharedFromThis<FCompElementCollectionViewModel>, public FEditorUndoClient
{
public:
	/**  
	 *	Factory method which creates a new FCompElementCollectionViewModel object.
	 *
	 *	@param	InElementsManager	The element management logic object
	 *	@param	InEditor			The UEditorEngine to register with (for undo/redo, etc.)
	 */
	static TSharedRef<FCompElementCollectionViewModel> Create(const TSharedRef<ICompElementManager>& InElementsManager, const TWeakObjectPtr<UEditorEngine>& InEditor)
	{
		TSharedRef<FCompElementCollectionViewModel> ElementsView(new FCompElementCollectionViewModel(InElementsManager, InEditor));
		ElementsView->Initialize();

		return ElementsView;
	}

	virtual ~FCompElementCollectionViewModel();

public:
	//~ Begin FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override { Refresh(); }
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	//~ End FEditorUndoClient interface

	/** 
	 *	Hook for the UI search box (and others) to filter the model's element list.
	 */
	void AddFilter(const TSharedRef<FCompElementFilter>& InFilter);

	/** 
	 *	Clears an existing filter from the elements list (refreshes the lists returned by GetCompShots() and GetElementsForComp()
	 */
	void RemoveFilter(const TSharedRef<FCompElementFilter>& InFilter);

	/** 
	 * Returns the (filtered) list of top-level compositing elements (for the UI to display).
	 */
	TArray< TSharedPtr<FCompElementViewModel> >& GetRootCompElements();

	/** 
	 * Returns a (filtered) list of child elements, nested directly under the specified CompItem.
	 */
	void GetChildElements(TSharedPtr<FCompElementViewModel> ParentPtr, TArray< TSharedPtr<FCompElementViewModel> >& OutChildElements);

	/** 
	 * Some elements are not selectable (like child actors). This determines that and returns the 
	 * (parent) element that should be selected instead.
	 * If the specified element is selectable, then this just returns that element.
	 */
	TSharedPtr<FCompElementViewModel> GetSelectionProxy(const TSharedPtr<FCompElementViewModel>& SelectedItem) const;

	/** Returns the a list of element model that are currently tracked as selected (should be reflected in the UI). */
	const TArray< TSharedPtr<FCompElementViewModel> >& GetSelectedElements() const;

	/** Appends the names of the currently selected elements to the provided array. */
	void GetSelectedElementNames(OUT TArray<FName>& OutSelectedElementNames) const;

	/** Sets the specified array of element objects as the currently selected elements (provides a way to sync with the UI). */
	void SetSelectedElements(const TArray< TSharedPtr<FCompElementViewModel> >& InSelectedElements);
	
	/** Sets the current selection to the specified element. */
	void SetSelectedElement(const FName& ElementName);

	/** Returns the bound UICommandList for the comp element view. */
	const TSharedRef<FUICommandList> GetCommandList() const;

	/********************************************************************
	 * EVENTS
	 ********************************************************************/

	/**	Broadcasts whenever one or more elements change. */
	DECLARE_DERIVED_EVENT(FCompElementCollectionViewModel, ICompElementManager::FOnElementsChanged, FOnElementsChanged);
	FOnElementsChanged& OnElementsChanged() { return ElementsChanged; }

	/**	Broadcasts whenever the currently selected elements change. */
	DECLARE_EVENT(FCompElementCollectionViewModel, FOnSelectionChanged);
	FOnSelectionChanged& OnSelectionChanged() { return SelectionChanged; }

	/**	Broadcasts whenever a rename is requested on the selected elements. */
	DECLARE_EVENT(FCompElementCollectionViewModel, FOnRenameRequested);
	FOnRenameRequested& OnRenameRequested() { return RenameRequested; }

private:
	/**  
	 *	Private constructor to force users to go through Create(), which properly initializes the model.
	 *
	 *	@param	InElementsManager	The element management logic object
	 *	@param	InEditor			The UEditorEngine to register with (for undo/redo, etc.)
	 */
	FCompElementCollectionViewModel(const TSharedRef<ICompElementManager>& InElementsManager, const TWeakObjectPtr<UEditorEngine>& InEditor);

	/** Initializes the elements view for use. */
	void Initialize();
	/** Binds all element browser commands to delegates. */
	void BindCommands();

	/**	Refreshes any cached information. */
	void Refresh();

	/** Handles updating the view-model when one of its filters changes. */
	void OnFilterChanged();
	/** */
	void OnElementsChanged(const ECompElementEdActions Action, const TWeakObjectPtr<ACompositingElement>& ChangedComp, const FName& ChangedProperty);
	/** */
	void OnElementAdded(const TWeakObjectPtr<ACompositingElement>& AddedElement);
	/** Handles updating the internal view-models when elements are deleted */
	void OnElementDelete();
	/** */
	void OnElementAttached(const TWeakObjectPtr<ACompositingElement>& AttachedElement);
	/** Refreshes the elements list */
	void OnResetElements();

	/** Discards any element view-models which are invalid */
	void DestructivelyPurgeInvalidViewModels(TArray< TWeakObjectPtr<ACompositingElement> >& InElements);
	/** Creates view-models for all elements in the specified list */
	void CreateViewModels(TArray< TWeakObjectPtr<ACompositingElement> >& ActualElements);
	/** Updates the view-model hierarchy of known elements. */
	void RebuildViewModelHierarchy();
	/** Rebuilds the list of filtered elements. */
	void RefreshFilteredElements();
	/**	Sorts the filtered elements list */
	void SortFilteredElements();

	/** Looks up the view-model associated with the specified element object. */
	TSharedPtr<FCompElementViewModel> GetViewModel(TWeakObjectPtr<ACompositingElement> ElementObj);
	/** Looks up the view-model associated with the specified element object. Returns false if it couldn't find one. */
	bool TryGetViewModel(TWeakObjectPtr<ACompositingElement> CompObjPtr, TSharedPtr<FCompElementViewModel>& OutViewModel);
	/** Returns a flat list of all element view-models (parents and children). */
	void GetAllViewModels(TArray< TSharedPtr<FCompElementViewModel> >& OutAllViewModels);

	/** Appends the selected element names to the specified array */
	void AppendSelectedElementNames(TArray<FName>& OutElementNames) const;
	/** Updates the element selection from a selection made in the level editor. */
	void OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);
	/** Updates actor selections from the internal selection state. */
	void RefreshActorSelections() const;
	
private:
	/** Creates a new top-level element, prompting the user to pick the class type first. */
	void CreateTopLevelElement_Executed();
	FName GenerateUniqueCompName() const;
	bool CreateTopLevelElement_CanExecute() const;

	/** Creates a new child element, nesting it under the currently selected element. */
	void CreateChildElement_Executed();
	FName GenerateUniqueElementName(TSubclassOf<ACompositingElement> ElementClass) const;
	bool CreateChildElement_CanExecute() const;
	
	/** Cuts the currently selected element */
	void CutElements_Executed();
	bool CutElements_CanExecute() const;

	/** Copies the currently selected elements */
	void CopyElements_Executed();
	bool CopyElements_CanExecute() const;

	/** Pastes the currently selected elements */
	void PasteElements_Executed();
	bool PasteElements_CanExecute() const;

	/** Duplicates the currently selected elements */
	void DuplicateElements_Executed();
	bool DuplicateElements_CanExecute() const;

	/** Deletes the currently selected elements */
	void DeleteElements_Executed();
	bool DeleteElements_CanExecute() const;

	/** Requests renaming of the selected element */
	void RequestRenameElement_Executed();
	bool RequestRenameElement_CanExecute() const;

	/** Opens a modeless dialog window, displaying the element's render result live. */
	void OpenPreview_Executed();
	bool OpenPreview_CanExecute() const;
	/** Resets/Rebuilds the element list (useful in case the list gets stale by an unaccounted problem). */
	void RefreshList_Executed();
	bool RefreshList_CanExecute() const;

private:
	/** The element management logic object. */
	const TSharedRef<ICompElementManager> CompElementManager;
	/** The UEditorEngine to use. */
	const TWeakObjectPtr<UEditorEngine> Editor;
	/** The list of commands with bound delegates for the element browser. */
	const TSharedRef<FUICommandList> CommandList;

	/** All top-level elements managed by the view. */
	TArray< TSharedPtr<FCompElementViewModel> > RootViewModels;

	/** The collection of filters used to restrict the elements shown in the view. */
	const TSharedRef<FCompElementFilterCollection> Filters;

	/** All elements shown in the view. */
	TArray< TSharedPtr<FCompElementViewModel> > FilteredRootItems;
	typedef TMultiMap< TSharedPtr<FCompElementViewModel>, TSharedPtr<FCompElementViewModel> > FFilteredChildList;
	FFilteredChildList FilteredChildren;

	/** Currently selected elements. */
	TArray< TSharedPtr<FCompElementViewModel> > SelectedElements;

	/**	Broadcasts whenever one or more elements change. */
	FOnElementsChanged ElementsChanged;
	/**	Broadcasts whenever the currently selected elements change. */
	FOnSelectionChanged SelectionChanged;
	/**	Broadcasts whenever a rename is requested on the selected elements. */
	FOnRenameRequested RenameRequested; 
};

