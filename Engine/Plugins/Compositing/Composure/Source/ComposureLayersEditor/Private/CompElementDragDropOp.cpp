// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompElementDragDropOp.h"

void FCompElementDragDropOp::Construct()
{
	const FSlateBrush* Icon = FAppStyle::GetBrush(TEXT("Layer.Icon16x"));
	if (Elements.Num() == 1)
	{
		FDecoratedDragDropOp::SetToolTip(FText::FromName(Elements[0]), Icon);
	}
	else
	{
		FText Text = FText::Format(NSLOCTEXT("FCompElementDragDropOp", "MultipleFormat", "{0} Elements"), Elements.Num());
		FDecoratedDragDropOp::SetToolTip(Text, Icon);
	}

	SetupDefaults();
	FDecoratedDragDropOp::Construct();
}
