// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DragAndDrop/ActorDragDropGraphEdOp.h"

class FDragDropEvent;

/** Drag/drop operation for dragging layers in the editor */
class FCompElementDragDropOp : public FActorDragDropGraphEdOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FCompElementDragDropOp, FActorDragDropGraphEdOp)

	virtual void Construct() override;

	/** The names of the layers being dragged */
	TArray<FName> Elements;
};
