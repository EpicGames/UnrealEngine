// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "CompElementEditorCommands"

/**
 * The set of commands supported by the compositing panel and associated windows.
 */
class FCompElementEditorCommands : public TCommands<FCompElementEditorCommands>
{

public:
	FCompElementEditorCommands() 
		: TCommands<FCompElementEditorCommands>("CompElementEditor", LOCTEXT("CompositingCmdsDesc", "Composure Compositing"), /*Parent =*/"GenericCommands", FAppStyle::GetAppStyleSetName())
	{
	}
	
	//~ Begin TCommands<> interface
	virtual void RegisterCommands() override
	{
		// Panel commands
		{
			UI_COMMAND(CreateEmptyComp, "Create New Comp", "Creates a new comp shot", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(CreateNewElement, "Add Layer Element", "Creates a new compositing element", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(RefreshCompList, "Refresh", "Refreshes the composure scene tree.", EUserInterfaceActionType::Button, FInputChord(EKeys::F5));
			UI_COMMAND(OpenElementPreview, "Preview", "Opens a preview window displaying the selected element.", EUserInterfaceActionType::Button, FInputChord(EKeys::P));

			UI_COMMAND(RemoveMediaOutput, "Remove", "Removes the media capture output from the selected element.", EUserInterfaceActionType::Button, FInputChord());
			UI_COMMAND(ResetMediaOutput, "Reset", "Opens a dialog to pick a new media output target.", EUserInterfaceActionType::Button, FInputChord());
		}
		
		// Preview window commands
		{
			UI_COMMAND(ResetColorPicker, "Reset", "Resets the color picker to its initial state.", EUserInterfaceActionType::Button, FInputChord(EKeys::C));
			UI_COMMAND(FreezeFrame, "Freeze Frame", "Toggles the current element's freeze framing.", EUserInterfaceActionType::Button, FInputChord(EKeys::F));

			UI_COMMAND(ToggleRedChannel, "Red", "Toggle the red channel.", EUserInterfaceActionType::ToggleButton, FInputChord());
			UI_COMMAND(ToggleGreenChannel, "Green", "Toggle the green channel.", EUserInterfaceActionType::ToggleButton, FInputChord());
			UI_COMMAND(ToggleBlueChannel, "Blue", "Toggle the blue channel.", EUserInterfaceActionType::ToggleButton, FInputChord());
			UI_COMMAND(ToggleAlphaChannel, "Alpha", "Toggle the alpha channel.", EUserInterfaceActionType::ToggleButton, FInputChord());
			UI_COMMAND(CycleChannelPresets, "Cycle Channel", "Cycle between RGBA channel presets", EUserInterfaceActionType::Button, FInputChord(EKeys::Tab));
			UI_COMMAND(SetChannelRed, "View Red Channel", "Enable red channel, disabling all others.", EUserInterfaceActionType::Button, FInputChord(EKeys::R));
			UI_COMMAND(SetChannelGreen, "View Green Channel", "Enable green channel, disabling all others.", EUserInterfaceActionType::Button, FInputChord(EKeys::G));
			UI_COMMAND(SetChannelBlue, "View Blue Channel", "Enable blue channel, disabling all others.", EUserInterfaceActionType::Button, FInputChord(EKeys::B));
			UI_COMMAND(SetChannelAlpha, "View Alpha Channel", "Enable alpha channel, disabling all others.", EUserInterfaceActionType::Button, FInputChord(EKeys::A));
		}		
	}
	//~ End TCommands<> interface

public:
	TSharedPtr<FUICommandInfo> CreateEmptyComp;
	TSharedPtr<FUICommandInfo> CreateNewElement;
	TSharedPtr<FUICommandInfo> RefreshCompList;
	TSharedPtr<FUICommandInfo> OpenElementPreview;

	TSharedPtr<FUICommandInfo> RemoveMediaOutput;
	TSharedPtr<FUICommandInfo> ResetMediaOutput;

	TSharedPtr<FUICommandInfo> ResetColorPicker;
	TSharedPtr<FUICommandInfo> FreezeFrame;

	TSharedPtr<FUICommandInfo> ToggleRedChannel;
	TSharedPtr<FUICommandInfo> ToggleGreenChannel;
	TSharedPtr<FUICommandInfo> ToggleBlueChannel;
	TSharedPtr<FUICommandInfo> ToggleAlphaChannel;
	TSharedPtr<FUICommandInfo> CycleChannelPresets;
	TSharedPtr<FUICommandInfo> SetChannelRed;
	TSharedPtr<FUICommandInfo> SetChannelGreen;
	TSharedPtr<FUICommandInfo> SetChannelBlue;
	TSharedPtr<FUICommandInfo> SetChannelAlpha;
};

#undef LOCTEXT_NAMESPACE
