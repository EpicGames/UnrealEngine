// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompElementEditorModule.h"
#include "CompElementManager.h"
#include "CompElementEditorCommands.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SCompElementBrowser.h"
#include "ComposureEditorStyle.h"
#include "LevelEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Application/SlateApplication.h"
#include "PropertyEditorModule.h"
#include "ComposureDetailCustomizations.h"
#include "EditorSupport/ICompositingEditor.h"
#include "Features/IModularFeatures.h"
#include "Widgets/SCompElementPreviewPane.h"
#include "Widgets/SCompElementPickerWindow.h"
#include "IAssetViewport.h"
#include "LevelEditorViewport.h"
#include "Widgets/SCompElementPreviewDialog.h"

LLM_DEFINE_TAG(Composure_ComposureLayersEditor);

namespace CompElementEditor_Impl
{
	static const FName ComposureLayersTabName(TEXT("ComposureLayers"));
	static const FName LevelEditorModuleName(TEXT("LevelEditor"));

	void RedrawViewport()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedPtr<IAssetViewport> Viewport = LevelEditorModule.GetFirstActiveViewport();
		if (Viewport.IsValid())
		{
			Viewport->GetAssetViewportClient().RedrawRequested(Viewport->GetActiveViewport());
		}
		else if (GCurrentLevelEditingViewportClient)
		{
			GCurrentLevelEditingViewportClient->RedrawRequested(nullptr);
		}
		else
		{
			GEditor->RedrawAllViewports(/*bInvalidateHitProxies =*/false);
		}
	}
}

/* FCompElementEditorModule
 *****************************************************************************/

class FCompElementEditorModule : public ICompElementEditorModule, public ICompositingEditor
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	//~ Begin ICompElementEditorModule interface
	virtual TSharedPtr<ICompElementManager> GetCompElementManager() override;
	virtual TArray<FCompEditorMenuExtender>& GetEditorMenuExtendersList() override;
	//~ End ICompElementEditorModule interface

	//~ Begin ICompositingEditor interface
	virtual TSharedPtr<SWidget> ConstructCompositingPreviewPane(TWeakUIntrfacePtr<ICompEditorImagePreviewInterface> PreviewTarget) override;
	virtual TSharedPtr<SWindow> RequestCompositingPickerWindow(TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget, const bool bAverageColorOnDrag, const FPickerResultHandler& OnPick, const FSimpleDelegate& OnCancel, const FText& WindowTitle) override;
	virtual bool DeferCompositingDraw(ACompositingElement* CompElement) override;
	virtual void RequestRedraw() override;
	//~ End ICompositingEditor interface

private:
	static TSharedRef<SDockTab> SpawnComposureLayersTab(const FSpawnTabArgs& SpawnTabArgs);

	void RegisterEditorTab();
	void ModulesChangedCallback(FName ModuleName, EModuleChangeReason ReasonForChange);

private:
	TArray<FCompEditorMenuExtender> EditorMenuExtenders;

	FDelegateHandle LevelEditorTabManagerChangedHandle;
	FDelegateHandle ModulesChangedHandle;

	TSharedPtr<ICompElementManager> CompElementManager;
};

//------------------------------------------------------------------------------
void FCompElementEditorModule::StartupModule() 
{
	LLM_SCOPE_BYTAG(Composure_ComposureLayersEditor);

	FComposureEditorStyle::Get();
	FCompElementEditorCommands::Register();

	// Details customizations
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.RegisterCustomClassLayout(TEXT("CompositingElement"), FOnGetDetailCustomizationInstance::CreateStatic(&FCompElementDetailsCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("CompositingMaterial"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCompositingMaterialPassCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("CompositingElementPass"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCompositingPassCustomization::MakeInstance));

	}

	CompElementManager = FCompElementManager::Create(GEditor);

	if (FModuleManager::Get().IsModuleLoaded(CompElementEditor_Impl::LevelEditorModuleName))
	{
		RegisterEditorTab();
	}
	else
	{
		ModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FCompElementEditorModule::ModulesChangedCallback);
	}
	IModularFeatures::Get().RegisterModularFeature(ICompositingEditor::GetModularFeatureName(), this);
}

//------------------------------------------------------------------------------
void FCompElementEditorModule::ShutdownModule()
{
	LLM_SCOPE_BYTAG(Composure_ComposureLayersEditor);

	IModularFeatures::Get().UnregisterModularFeature(ICompositingEditor::GetModularFeatureName(), this);

	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterTabSpawner(CompElementEditor_Impl::ComposureLayersTabName);
	}

	FModuleManager::Get().OnModulesChanged().Remove(ModulesChangedHandle);

	if (LevelEditorTabManagerChangedHandle.IsValid() && FModuleManager::Get().IsModuleLoaded(CompElementEditor_Impl::LevelEditorModuleName))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(CompElementEditor_Impl::LevelEditorModuleName);
		LevelEditorModule.OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
	}

	// Details customizations
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("CompositingElementPass"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("CompositingMaterial"));
		PropertyModule.UnregisterCustomClassLayout(TEXT("CompositingElement"));		
	}

	FCompElementEditorCommands::Unregister();
}

//------------------------------------------------------------------------------
TSharedPtr<ICompElementManager> FCompElementEditorModule::GetCompElementManager()
{
	return CompElementManager;
}

//------------------------------------------------------------------------------
TArray<ICompElementEditorModule::FCompEditorMenuExtender>& FCompElementEditorModule::GetEditorMenuExtendersList()
{
	return EditorMenuExtenders;
}

//------------------------------------------------------------------------------
TSharedPtr<SWidget> FCompElementEditorModule::ConstructCompositingPreviewPane(TWeakUIntrfacePtr<ICompEditorImagePreviewInterface> PreviewTarget)
{
	TSharedPtr<SCompElementPreviewPane> PreviewPane;
	SAssignNew(PreviewPane, SCompElementPreviewPane)
		.PreviewTarget(PreviewTarget)
		.OnRedraw_Lambda([]()
			{
				CompElementEditor_Impl::RedrawViewport();
			}
		)
		.OverlayExtender_Lambda([PreviewTarget](TSharedRef<SOverlay> Overlay)
			{	
				TSharedPtr<SButton> MaximizeButton;

				Overlay->AddSlot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Top)
				[
					SAssignNew(MaximizeButton, SButton)
						.ContentPadding(0)
						.ButtonStyle(FAppStyle::Get(), "ToggleButton")
						.Cursor(EMouseCursor::Default)
						.ToolTipText(NSLOCTEXT("FCompElementEditorModule", "MaximizePreviewTooltip", "Maximize"))
						.OnClicked_Lambda([PreviewTarget, Overlay]()->FReply
						{
							FText WindowTitle;
							if (PreviewTarget.IsValid())
							{
								if ( UActorComponent* TargetComp = Cast<UActorComponent>(PreviewTarget.GetObject().Get()) )
								{
									if (ACompositingElement* AsElement = Cast<ACompositingElement>(TargetComp->GetOwner()))
									{
										WindowTitle = FText::Format(NSLOCTEXT("FCompElementEditorModule", "PreviewTitle", "Preview: {0}"), FText::FromName(AsElement->GetCompElementName()));
									}
								}
							}

							SCompElementPreviewDialog::OpenPreviewWindow(PreviewTarget, Overlay, WindowTitle);
							return FReply::Handled();
						})
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Top)
						.Content()
						[
							SNew(SImage)
								.Image(FComposureEditorStyle::Get().GetBrush("CompPreviewPane.MaximizeWindow16x"))
								.ColorAndOpacity_Lambda([MaximizeButton]()->FSlateColor
								{
									return MaximizeButton.IsValid() && MaximizeButton->IsHovered() ? FLinearColor(0.75f, 0.75f, 0.75f, 1.f) : FLinearColor(0.75f, 0.75f, 0.75f, 0.75f);
								})
						]
				];
			}
		);

	return PreviewPane;
}

//------------------------------------------------------------------------------
TSharedPtr<SWindow> FCompElementEditorModule::RequestCompositingPickerWindow(TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget, const bool bAverageColorOnDrag, const FPickerResultHandler& OnPick, const FSimpleDelegate& OnCancel, const FText& WindowTitle)
{
	FCompElementColorPickerArgs PickerArgs;
	PickerArgs.PickerTarget = PickerTarget;
	PickerArgs.OnColorPicked = OnPick;;
	PickerArgs.OnColorPickerCanceled = OnCancel;
	PickerArgs.ParentWidget = FSlateApplication::Get().GetActiveTopLevelWindow();
	PickerArgs.bAverageColorOnDrag = bAverageColorOnDrag;
	PickerArgs.WindowTitle = WindowTitle;

	return SCompElementPickerWindow::Open(PickerArgs);

}

//------------------------------------------------------------------------------
bool FCompElementEditorModule::DeferCompositingDraw(ACompositingElement* CompElement)
{
	static IConsoleVariable* CVarUsingDecoupledDrawing = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Composure.CompositingElements.Editor.DecoupleRenderingFromLevelViewport"));
	if (CompElement && CVarUsingDecoupledDrawing && CVarUsingDecoupledDrawing->GetInt() > 0 && !CompElementManager->IsDrawing(CompElement))
	{
		UWorld* World = CompElement->GetWorld();
		if (World && World->WorldType == EWorldType::Editor)
		{
			CompElementManager->RequestRedraw();
			return true;
		}
	}
	return false;
}

//------------------------------------------------------------------------------
void FCompElementEditorModule::RequestRedraw()
{
	CompElementManager->RequestRedraw();
}

//------------------------------------------------------------------------------
TSharedRef<SDockTab> FCompElementEditorModule::SpawnComposureLayersTab(const FSpawnTabArgs& /*SpawnTabArgs*/)
{
	const TSharedRef<SDockTab> MajorTab =
		SNew(SDockTab)
		//.Icon(IconBrush)
		.TabRole(ETabRole::NomadTab);

	MajorTab->SetContent(SNew(SCompElementBrowser));

	return MajorTab;
}

//------------------------------------------------------------------------------
void FCompElementEditorModule::RegisterEditorTab()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(CompElementEditor_Impl::LevelEditorModuleName);

	LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda([]()
	{
		LLM_SCOPE_BYTAG(Composure_ComposureLayersEditor);
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(CompElementEditor_Impl::LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		const FSlateIcon LayersIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.ComposureCompositing");
		LevelEditorTabManager->RegisterTabSpawner(CompElementEditor_Impl::ComposureLayersTabName, FOnSpawnTab::CreateStatic(&FCompElementEditorModule::SpawnComposureLayersTab))
			.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorComposureLayerBrowser", "Legacy Composure"))
			.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "LevelEditorComposureLayerBrowserTooltipText", "Open the legacy Composure compositing tab."))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory())
			.SetIcon(LayersIcon);
	});
}

//------------------------------------------------------------------------------
void FCompElementEditorModule::ModulesChangedCallback(FName ModuleName, EModuleChangeReason ReasonForChange)
{
	if (ReasonForChange == EModuleChangeReason::ModuleLoaded && ModuleName == CompElementEditor_Impl::LevelEditorModuleName)
	{
		RegisterEditorTab();
	}
}

IMPLEMENT_MODULE(FCompElementEditorModule, ComposureLayersEditor);

/* ICompElementEditorModule
 *****************************************************************************/

//------------------------------------------------------------------------------
ICompElementEditorModule& ICompElementEditorModule::Get()
{
	return FModuleManager::GetModuleChecked<ICompElementEditorModule>(TEXT("ComposureLayersEditor"));
}
