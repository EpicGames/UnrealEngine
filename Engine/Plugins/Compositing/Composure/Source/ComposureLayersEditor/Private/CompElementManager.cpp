// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompElementManager.h"
#include "Templates/SharedPointer.h"
#include "CompositingElement.h"
#include "Engine/Level.h"
#include "Selection.h"
#include "UObject/UObjectIterator.h"
#include "CompositingElements/CompositingElementOutputs.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "MediaOutput.h"
#include "ScopedWorldLevelContext.h"
#include "Misc/IFilter.h"
#include "EditorCompElementContainer.h"
#include "CompositingViewportClient.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "CompElementManager"

FCompElementManager::FCompElementManager(const TWeakObjectPtr<UEditorEngine>& InEditor)
	: Editor(InEditor)
	, ElementsContainer(nullptr)
{
	ensure(InEditor.IsValid());
}

FCompElementManager::~FCompElementManager()
{
	if (Editor.IsValid())
	{
		Editor->OnLevelActorDeleted().RemoveAll(this);
		Editor->OnLevelActorAdded().RemoveAll(this);
		Editor->OnLevelActorListChanged().RemoveAll(this);
		Editor->OnWorldDestroyed().RemoveAll(this);
		Editor->OnWorldAdded().RemoveAll(this);

		Editor->OnBlueprintCompiled().RemoveAll(this);
	}

	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

	FEditorDelegates::MapChange.RemoveAll(this);
}

void FCompElementManager::Initialize()
{
	ElementsContainer = NewObject<UEditorCompElementContainer>(GetTransientPackage(), NAME_None, RF_Transactional);

	if (FSlateApplication::IsInitialized())
	{
		EditorCompositingViewport = FCompositingViewport::Create(ElementsContainer);
	}

	FEditorDelegates::MapChange.AddRaw(this, &FCompElementManager::OnEditorMapChange);
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FCompElementManager::OnWorldAdded);
	FWorldDelegates::LevelAddedToWorld.AddRaw(this, &FCompElementManager::OnWorldLevelsChange);
	FWorldDelegates::LevelRemovedFromWorld.AddRaw(this, &FCompElementManager::OnWorldLevelsChange);

	if (ensure(Editor.IsValid()))
	{
		Editor->OnWorldAdded().AddRaw(this, &FCompElementManager::OnWorldAdded);
		Editor->OnWorldDestroyed().AddRaw(this, &FCompElementManager::OnWorldRemoved);

		Editor->OnLevelActorListChanged().AddRaw(this, &FCompElementManager::OnLevelActorsListChange);

		Editor->OnLevelActorAdded().AddRaw(this, &FCompElementManager::OnLevelActorAdded);
		Editor->OnLevelActorDeleted().AddRaw(this, &FCompElementManager::OnLevelActorRemoved);
		Editor->OnBlueprintCompiled().AddRaw(this, &FCompElementManager::OnBlueprintCompiled);
	}
}

TWeakObjectPtr<ACompositingElement> FCompElementManager::CreateElement(const FName& ElementName, TSubclassOf<ACompositingElement> ClassType, AActor* LevelContext, EObjectFlags ObjectFlags)
{
	ACompositingElement* SpawnedActor = nullptr;

	UWorld* TargetWorld = nullptr;
	if (LevelContext)
	{
		TargetWorld = LevelContext->GetWorld();
	}
	else
	{
		for (TWeakObjectPtr<ACompositingElement> ElementPtr : *ElementsContainer)
		{
			if (ElementPtr.IsValid())
			{
				TargetWorld = ElementPtr->GetWorld();
				LevelContext = ElementPtr.Get();
				break;
			}
		}
	}
	if (!TargetWorld)
	{
		TargetWorld = this->GetWorld();
	}

	if (TargetWorld != nullptr)
	{
		FScopedWorldLevelContext ScopedLevelContext(TargetWorld, LevelContext);

		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = ElementName;
		SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		SpawnParams.ObjectFlags |= ObjectFlags;
		SpawnedActor = TargetWorld->SpawnActor<ACompositingElement>(ClassType, FTransform::Identity, SpawnParams);

		if (SpawnedActor)
		{
			SpawnedActor->SetElementName(ElementName);
		}
	}

	return SpawnedActor;
}

TWeakObjectPtr<ACompositingElement> FCompElementManager::GetElement(const FName& ElementName) const
{
	for (TWeakObjectPtr<ACompositingElement> Element : *ElementsContainer)
	{
		if (Element.IsValid() && Element->GetCompElementName() == ElementName)
		{
			return Element;
		}
	}

	return nullptr;
}

bool FCompElementManager::TryGetElement(const FName& ElementName, TWeakObjectPtr<ACompositingElement>& OutElement)
{
	OutElement = GetElement(ElementName);
	return OutElement != nullptr;
}

TArray<FName> FCompElementManager::FindNamesOfAllChildElements(FName ElementName) const
{
	TArray<FName> Output;
	TWeakObjectPtr<ACompositingElement> Root = GetElement(ElementName);

	if (Root.IsValid())
	{
		for (ACompositingElement* Child: Root->GetChildElements())
		{
			if (Child)
			{
				Output.Append(FindNamesOfAllChildElements(Child->GetCompElementName()));
				Output.Add(Child->GetCompElementName());
			}
		}
	}
	return Output;
}

void FCompElementManager::AddAllCompElementsTo(TArray< TWeakObjectPtr<ACompositingElement> >& OutElements) const
{
	OutElements = *ElementsContainer;
}

void FCompElementManager::DeleteElementAndChildren(const FName& ElementToDelete, bool bIsCalledFromEditor)
{
	TArray<FName> NameCollection = FindNamesOfAllChildElements(ElementToDelete);
	NameCollection.Add(ElementToDelete);
	DeleteElements(NameCollection,bIsCalledFromEditor);
}

void FCompElementManager::DeleteElements(const TArray<FName>& ElementsToDelete, bool bIsCalledFromEditor)
{
	ensure(PendingDeletion.Num() == 0);

	//If called from BP/C++, deselect all the actors first to avoid accidentally delete any actor.
	if (!bIsCalledFromEditor)
	{
		Editor->SelectNone(/*bNoteSelectionChange=*/false,/*bDeselectBSPSurfs=*/false,/*WarnAboutManyActors=*/false);
	}
	
	TArray< TWeakObjectPtr<ACompositingElement> > ValidElementsToDelete;
	ValidElementsToDelete.Reserve(ElementsToDelete.Num());
	for (const FName& ElementName : ElementsToDelete)
	{
		TWeakObjectPtr<ACompositingElement> Element;
		if (TryGetElement(ElementName, OUT Element))
		{
			ValidElementsToDelete.Add(Element);
		}
	}

	TArray< TWeakObjectPtr<UObject> > ExcessSelectedObjs;
	int32 SelectionCount = 0;
	for (FSelectionIterator SelectionIt(Editor->GetSelectedActorIterator()); SelectionIt; ++SelectionIt, ++SelectionCount)
	{
		TWeakObjectPtr<UObject> SelectedObjPtr(*SelectionIt);
		if (!ValidElementsToDelete.Contains(SelectedObjPtr))
		{
			ExcessSelectedObjs.Add(*SelectionIt);
		}
	}

	//No need to ensure excess selected objects if this function is not called from editor
	if (bIsCalledFromEditor)
	{
		ensure(ExcessSelectedObjs.Num() == 0);
	}
	
	TArray< TWeakObjectPtr<ACompositingElement> > OldParents;
	OldParents.Reserve(ValidElementsToDelete.Num());
	PendingDeletion.Reserve(ValidElementsToDelete.Num());

	for (const TWeakObjectPtr<ACompositingElement>& Element : ValidElementsToDelete)
	{
		PendingDeletion.Add(Element);
		if (!Element->IsSubElement() || ValidElementsToDelete.Contains(Element->GetParentActor()))
		{
			OldParents.Add(nullptr);
		}
		else
		{
			ACompositingElement* ElementParent = Element->GetElementParent();
			OldParents.Add(ElementParent);

			// remove to avoid "actor is referenced by other actors" warning
			Element->Modify();
			ElementParent->Modify();
			ElementParent->DetatchAsChildLayer(Element.Get());
		}
	}

	if (ExcessSelectedObjs.Num() > 0 || ValidElementsToDelete.Num() != SelectionCount)
	{
		USelection* EdSelectionManager = Editor->GetSelectedActors();
		EdSelectionManager->BeginBatchSelectOperation();
		for (const TWeakObjectPtr<ACompositingElement>& Element : ValidElementsToDelete)
		{
			EdSelectionManager->Modify();
			Editor->SelectActor(Element.Get(), /*bSelect =*/true, /*bNotifyForActor =*/false, /*bSelectEvenIfHidden =*/true);
		}
		EdSelectionManager->EndBatchSelectOperation();
	}

	if (UWorld* World = GetWorld())
	{
		//Disable prompt window if this function is called directly from BP/C++
		Editor->edactDeleteSelected(World, /*bVerifyDeletionCanHappen =*/true,/*bWarnAboutReferences=*/bIsCalledFromEditor,/*bWarnAboutSoftReferences=*/bIsCalledFromEditor);
	}

	for (const TWeakObjectPtr<ACompositingElement>& Element : PendingDeletion)
	{
		if (Element.IsValid())
		{
			const int32 ElementIndex = ValidElementsToDelete.Find(Element);
			if (ElementIndex != INDEX_NONE)
			{
				const TWeakObjectPtr<ACompositingElement>& ElementParent = OldParents[ElementIndex];
				if (ElementParent.IsValid())
				{
					ElementParent->Modify();
					ElementParent->AttachAsChildLayer(Element.Get());
				}
			}
		}
	}
	PendingDeletion.Empty();

	// @TODO: restore selection

	CompsChanged.Broadcast(ECompElementEdActions::Delete, nullptr, NAME_None);
	RequestRedraw();
}

bool FCompElementManager::RenameElement(const FName OriginalElementName, const FName& NewElementName)
{
	// We specifically don't pass the original ElementName by reference to avoid it changing
	// it's original value, in case, it would be the reference of the Element's actual FName
	if (OriginalElementName == NewElementName)
	{
		return false;
	}

	TWeakObjectPtr<ACompositingElement> Element;
	if (!TryGetElement(OriginalElementName, Element))
	{
		return false;
	}

	Element->Modify();
	Element->SetElementName(NewElementName);

	CompsChanged.Broadcast(ECompElementEdActions::Rename, Element, "CompShotIdName");

	RequestRedraw();

	return true;
}

bool FCompElementManager::AttachCompElement(const FName ParentName, const FName ElementName)
{
	bool bChangesOccurred = false;

	TWeakObjectPtr<ACompositingElement> FoundParent;
	TWeakObjectPtr<ACompositingElement> FoundElement;

	if (TryGetElement(ParentName, FoundParent) && TryGetElement(ElementName, FoundElement))
	{
		if (FoundParent.IsValid() && FoundElement.IsValid())
		{
			const bool bIsParentChildActor = FoundParent->GetParentComponent() != nullptr;
			const bool bIsElementChildActor = FoundElement->GetParentComponent() != nullptr;

			if (!bIsParentChildActor && !bIsElementChildActor)
			{
				if (FoundParent->GetLevel() == FoundElement->GetLevel())
				{
					FoundParent->Modify();
					FoundElement->Modify();

					bChangesOccurred = FoundParent->AttachAsChildLayer(FoundElement.Get());
				}
				else
				{
					// @TODO: Log/Toast an error - preventing elements from different levels linking together
				}
			}
		}
	}

	if (bChangesOccurred)
	{
		//TODO: Call full reset from View Model so we don't need to here
		//CompsChanged.Broadcast(ECompElementEdActions::Attached, FoundElement, NAME_None);
		CompsChanged.Broadcast(ECompElementEdActions::Reset, nullptr, NAME_None);
	}

	return bChangesOccurred;
}

bool FCompElementManager::SelectElementActors(const TArray<FName>& ElementNames, bool bSelect, bool bNotify, bool bSelectEvenIfHidden, const TSharedPtr<FActorFilter>& Filter)
{
	if (ElementNames.Num() == 0)
	{
		return true;
	}

	Editor->GetSelectedActors()->BeginBatchSelectOperation();
	bool bChangesOccurred = false;

	for (const TWeakObjectPtr<ACompositingElement>& ElementPtr : *ElementsContainer)
	{
		if (!ElementPtr.IsValid())
		{
			continue;
		}

		ACompositingElement* Element = ElementPtr.Get();

		if (!ElementNames.Contains(Element->GetCompElementName()))
		{
			continue;
		}

		if (Filter.IsValid() && !Filter->PassesFilter(Element))
		{
			continue;
		}

		Editor->GetSelectedActors()->Modify();
		Editor->SelectActor(Element, bSelect, /*bNotifyForActor =*/false, bSelectEvenIfHidden);
		bChangesOccurred = true;
	}

	Editor->GetSelectedActors()->EndBatchSelectOperation();

	if (bNotify)
	{
		Editor->NoteSelectionChange();
	}

	return bChangesOccurred;
}

void FCompElementManager::ToggleElementRendering(const FName& ElementName)
{
	const TWeakObjectPtr<ACompositingElement> Element = EnsureElementExists(ElementName);

	if (ensure(Element != nullptr))
	{
		Element->Modify();
		Element->SetAutoRun(!Element->bAutoRun);

		CompsChanged.Broadcast(ECompElementEdActions::Modify, Element, TEXT("bAutoRun"));
	}
}

void FCompElementManager::ToggleElementFreezeFrame(const FName& ElementName)
{
	const TWeakObjectPtr<ACompositingElement> Element = EnsureElementExists(ElementName);

	if (ensure(Element != nullptr))
	{
		Element->Modify();

		ETargetUsageFlags FreezeFlags = ETargetUsageFlags::USAGE_Input | ETargetUsageFlags::USAGE_Transform;
		if (Element->FreezeFrameController.HasAllFlags(FreezeFlags))
		{
			if (Element->FreezeFrameController.ClearFreezeFlags())
			{
				RequestRedraw();
			}
		}
		else
		{
			Element->FreezeFrameController.SetFreezeFlags(FreezeFlags);
		}

		CompsChanged.Broadcast(ECompElementEdActions::Modify, Element, TEXT("FreezeFrameMask"));
	}
}

void FCompElementManager::ToggleMediaCapture(const FName& ElementName)
{
	TWeakObjectPtr<ACompositingElement> FoundComp;
	if (TryGetElement(ElementName, FoundComp) && FoundComp.IsValid())
	{
		UCompositingMediaCaptureOutput* MediaOutputPass = Cast<UCompositingMediaCaptureOutput>(FoundComp->FindOutputPass(UCompositingMediaCaptureOutput::StaticClass()));
		if (!MediaOutputPass || !MediaOutputPass->CaptureOutput)
		{
			MediaOutputPass = ResetMediaCapture(ElementName);
			MediaOutputPass->SetPassEnabled(true);
			RequestRedraw();
		}
		else
		{
			const bool bRequestRedraw = !MediaOutputPass->IsPassEnabled();

			MediaOutputPass->Modify();
			MediaOutputPass->SetPassEnabled(!MediaOutputPass->IsPassEnabled());

			if (bRequestRedraw)
			{
				RequestRedraw();
			}
		}
	}
}

UCompositingMediaCaptureOutput* FCompElementManager::ResetMediaCapture(const FName& ElementName)
{
	TWeakObjectPtr<ACompositingElement> FoundComp;
	if (TryGetElement(ElementName, FoundComp) && FoundComp.IsValid())
	{
		UCompositingMediaCaptureOutput* MediaOutputPass = Cast<UCompositingMediaCaptureOutput>(FoundComp->FindOutputPass(UCompositingMediaCaptureOutput::StaticClass()));
		if (!MediaOutputPass)
		{
			FoundComp->Modify();
			MediaOutputPass = FoundComp->AddNewPass<UCompositingMediaCaptureOutput>(TEXT("MediaCapture"));
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

		FOpenAssetDialogConfig SelectAssetConfig;
		SelectAssetConfig.DialogTitleOverride = LOCTEXT("ChooseMediaOutputTitle", "Choose a media output");
		SelectAssetConfig.bAllowMultipleSelection = false;
		SelectAssetConfig.DefaultPath = TEXT("/Game");

		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			if (ClassIt->IsChildOf(UMediaOutput::StaticClass()) && !ClassIt->HasAnyClassFlags(CLASS_Abstract))
			{
				SelectAssetConfig.AssetClassNames.Add(ClassIt->GetClassPathName());
			}
		}

		TArray<FAssetData> AssetData = ContentBrowserModule.Get().CreateModalOpenAssetDialog(SelectAssetConfig);
		if (AssetData.Num() > 0)
		{
			UMediaOutput* MediaOutputAsset = Cast<UMediaOutput>(AssetData[0].GetAsset());
			if (MediaOutputAsset)
			{
				MediaOutputPass->Modify();
				MediaOutputPass->CaptureOutput = MediaOutputAsset;
			}
		}

		return MediaOutputPass;
	}

	return nullptr;
}

void FCompElementManager::RemoveMediaCapture(const FName& ElementName)
{
	TWeakObjectPtr<ACompositingElement> FoundComp;
	if (TryGetElement(ElementName, FoundComp) && FoundComp.IsValid())
	{
		FoundComp->Modify();
		FoundComp->RemovePassesOfType(UCompositingMediaCaptureOutput::StaticClass());
	}
}

void FCompElementManager::RefreshElementsList()
{
	if (ElementsContainer)
	{
		ElementsContainer->RebuildEditorElementsList();
	}
	CompsChanged.Broadcast(ECompElementEdActions::Reset, nullptr, NAME_None);
}

void FCompElementManager::RequestRedraw()
{
	if (!EditorCompositingViewport.IsValid() && FSlateApplication::IsInitialized())
	{
		EditorCompositingViewport = FCompositingViewport::Create(ElementsContainer);
	}

	if (EditorCompositingViewport.IsValid())
	{
		EditorCompositingViewport->RequestRedraw();
	}
}

bool FCompElementManager::IsDrawing(ACompositingElement* CompElement) const
{
	if (CompElement && EditorCompositingViewport.IsValid() && EditorCompositingViewport->IsDrawing())
	{
		return ElementsContainer && ElementsContainer->Contains(CompElement);
	}
	return false;
}

void FCompElementManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ElementsContainer);
}

UWorld* FCompElementManager::GetWorld() const
{
	UWorld* TargetWorld = ElementsContainer->GetWorld();
	if (!TargetWorld && Editor.IsValid())
	{
		TargetWorld = Editor->GetEditorWorldContext().World();
	}
	return TargetWorld;
}

TWeakObjectPtr<ACompositingElement> FCompElementManager::EnsureElementExists(const FName& ElementName)
{
	TWeakObjectPtr<ACompositingElement> Element;
	if (!TryGetElement(ElementName, Element))
	{
		Element = CreateElement(ElementName, ACompositingElement::StaticClass());
	}

	return Element;
}

void FCompElementManager::OnLevelActorAdded(AActor* InActor)
{
	if (ACompositingElement* AsCompElement = Cast<ACompositingElement>(InActor))
	{
		const bool bAdded = ElementsContainer->Add(AsCompElement);
		if (bAdded)
		{

			AsCompElement->OnConstructed.AddRaw(this, &FCompElementManager::OnCompElementConstructed);

			TWeakObjectPtr<ACompositingElement> NewCompPtr(AsCompElement);
			CompsChanged.Broadcast(ECompElementEdActions::Add, NewCompPtr, NAME_None);
		}
	}
}

void FCompElementManager::OnLevelActorRemoved(AActor* InActor)
{
	ACompositingElement* ElementActor = Cast<ACompositingElement>(InActor);
	if (ElementActor)
	{
		if (!GIsReinstancing)
		{
			if (ACompositingElement* Parent = ElementActor->GetElementParent())
			{
				Parent->Modify();
				Parent->DetatchAsChildLayer(ElementActor);
			}

			for (ACompositingElement* Child : ElementActor->GetChildElements())
			{
				if (Child)
				{
					Child->Modify();
					ElementActor->DetatchAsChildLayer(Child);
				}
			}
		}
		PendingDeletion.Remove(ElementActor);
		bool bRemoved = ElementsContainer->Remove(ElementActor);

		if (bRemoved)
		{
			CompsChanged.Broadcast(ECompElementEdActions::Delete, nullptr, NAME_None);
		}
	}
	if (GIsReinstancing)
	{
		CompsChanged.Broadcast(ECompElementEdActions::Reset, nullptr, NAME_None);
	}
}

void FCompElementManager::OnBlueprintCompiled()
{
	CompsChanged.Broadcast(ECompElementEdActions::Reset, nullptr, NAME_None);
}

void FCompElementManager::OnCompElementConstructed(ACompositingElement* ConstructedElement)
{
	ConstructedElement->OnConstructed.RemoveAll(this);

	if (UChildActorComponent* ChildActorComp = ConstructedElement->GetParentComponent())
	{
		// @TODO: this reset only needs to happen once for the whole actor, but as far as I can tell
		//        there's no hook for after an actor and all its children have been constructed
		CompsChanged.Broadcast(ECompElementEdActions::Reset, ConstructedElement, NAME_None);
	}
}

void FCompElementManager::OnEditorMapChange(uint32 /*MapChangeFlags*/)
{
	RefreshElementsList();
}

void FCompElementManager::OnWorldAdded(UWorld* InWorld)
{
	if (InWorld && InWorld->WorldType == EWorldType::Editor)
	{
		RefreshElementsList();
	}
}

void FCompElementManager::OnWorldRemoved(UWorld* InWorld)
{
	if (!InWorld || InWorld->WorldType == EWorldType::Editor)
	{
		RefreshElementsList();
	}
}

void FCompElementManager::OnWorldLevelsChange(ULevel* /*InLevel*/, UWorld* InWorld)
{
	if (InWorld && InWorld->WorldType == EWorldType::Editor)
	{
		RefreshElementsList();
	}
}

void FCompElementManager::OnLevelActorsListChange()
{
	RefreshElementsList();
}

void FCompElementManager::OnCreateNewElement(AActor* NewElement)
{
	OnLevelActorAdded(NewElement);
	RefreshElementsList();
}

void FCompElementManager::OnDeleteElement(AActor* ElementToDelete)
{
	OnLevelActorRemoved(ElementToDelete);
	RefreshElementsList();
}



#undef LOCTEXT_NAMESPACE
