// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICompElementManager.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtr.h"

class UEditorCompElementContainer;
class AActor;
class UPackage;
class UEditorEngine;
class FCompositingViewport;
class ULevel;

/** Management class for interfacing with editor compositing element objects. */
class FCompElementManager : public ICompElementManager, public FGCObject
{
public:
	/**  
	 *	Factory method which creates a new FCompElementManager object.
	 *
	 *	@param  InEditor	The UEditorEngine to register delegates with, etc.
	 */
	static TSharedRef<FCompElementManager> Create(const TWeakObjectPtr<UEditorEngine>& InEditor)
	{
		TSharedRef<FCompElementManager> CompShots = MakeShareable( new FCompElementManager(InEditor) );
		CompShots->Initialize();
		return CompShots;
	}

	virtual ~FCompElementManager();

public: 
	//~ Begin ICompElementManager interface
	virtual TWeakObjectPtr<ACompositingElement> CreateElement(const FName& ElementName, TSubclassOf<ACompositingElement> ClassType, AActor* LevelContext = nullptr, EObjectFlags ObjectFlags = EObjectFlags::RF_NoFlags) override;
	virtual TWeakObjectPtr<ACompositingElement> GetElement(const FName& ElementName) const override;
	virtual bool TryGetElement(const FName& ElementName, TWeakObjectPtr<ACompositingElement>& OutElement) override;
	virtual void AddAllCompElementsTo(TArray< TWeakObjectPtr<ACompositingElement> >& OutElements) const override;
	virtual void DeleteElementAndChildren(const FName& ElementToDelete, bool isCalledFromEditor = true) override;
	virtual void DeleteElements(const TArray<FName>& ElementsToDelete, bool isCalledFromEditor = true) override;
	virtual bool RenameElement(const FName OriginalElementName, const FName& NewElementName) override;
	virtual bool AttachCompElement(const FName ParentName, const FName ElementName) override;
	virtual bool SelectElementActors(const TArray<FName>& ElementNames, bool bSelect, bool bNotify, bool bSelectEvenIfHidden = false, const TSharedPtr<FActorFilter>& Filter = TSharedPtr<FActorFilter>(nullptr)) override;
	virtual void ToggleElementRendering(const FName& ElementName) override;
	virtual void ToggleElementFreezeFrame(const FName& ElementName) override;
	virtual void ToggleMediaCapture(const FName& ElementName) override;
	virtual UCompositingMediaCaptureOutput* ResetMediaCapture(const FName& ElementName) override;
	virtual void RemoveMediaCapture(const FName& ElementName) override; 	
	virtual void RefreshElementsList() override;
	void RequestRedraw() override;
	virtual bool IsDrawing(ACompositingElement* CompElement) const override;
	virtual void OnCreateNewElement(AActor* NewElement) override;
	virtual void OnDeleteElement(AActor* ElementToDelete) override;

	DECLARE_DERIVED_EVENT(FCompElementManager, ICompElementManager::FOnElementsChanged, FOnElementsChanged);
	virtual FOnElementsChanged& OnElementsChanged() override { return CompsChanged; }
	//~ End ICompElementManager interface

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCompElementManager");
	}
	//~ End FGCObject interface
	
private:
	/** Private constructor to force users to go through Create(), which properly initializes the manager. */
	FCompElementManager(const TWeakObjectPtr<UEditorEngine>& InEditor);

	/**	Prepares for use. */
	void Initialize();
	/** */
	UWorld* GetWorld() const;

	/** Utility function for looking up and creating a element if it doesn't already exist. */
	TWeakObjectPtr<ACompositingElement> EnsureElementExists(const FName& ElementName);

	/**
	 * Find all the child elements' FName given a parent element. 
	 *
	 * @param  ElementName	        The name of the parent element.
	 * @return TArray<FName>        Array containing all the child elements' FName.
	 */
	TArray<FName> FindNamesOfAllChildElements(FName ElementName) const;

	void OnLevelActorAdded(AActor* InActor);
	void OnLevelActorRemoved(AActor* InActor);
	void OnBlueprintCompiled();
	void OnCompElementConstructed(ACompositingElement* ConstructedElement);
	void OnEditorMapChange(uint32 MapChangeFlags);
	void OnWorldAdded(UWorld* NewWorld);
	void OnWorldRemoved(UWorld* NewWorld);
	void OnWorldLevelsChange(ULevel* InLevel, UWorld* InWorld);
	void OnLevelActorsListChange();

private:
	/** The associated UEditorEngine to bind/un-bind with. */
	TWeakObjectPtr<UEditorEngine> Editor;

	/** UObject in charge of tracking all editor element actors. Separated as a UObject to more easily facilitate undo/redo actions. */
	TObjectPtr<UEditorCompElementContainer> ElementsContainer;
	/** Hidden editor viewport, in charge on enqueuing compositing render commands. */
	TSharedPtr<FCompositingViewport> EditorCompositingViewport;
	
	/** 
	 * List which tracks elements queued for delete - used to circumvent broken 
	 * child/parent link warnings. Tracked in case the user cancels the delete op and we need to restore the links. 
	 */
	TArray< TWeakObjectPtr<ACompositingElement> > PendingDeletion;

	/** Event broadcasted whenever one or more elements are modified. */
	FOnElementsChanged CompsChanged;
};
