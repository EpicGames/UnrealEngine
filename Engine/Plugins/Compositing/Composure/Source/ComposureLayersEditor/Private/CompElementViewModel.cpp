// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompElementViewModel.h"
#include "CompositingElement.h"
#include "CompElementEditorCommands.h"
#include "CompositingElements/CompositingElementOutputs.h"
#include "Framework/Commands/UICommandList.h"
#include "ScopedTransaction.h"
#include "ICompElementManager.h"

#define LOCTEXT_NAMESPACE "CompElement"

FCompElementViewModel::FCompElementViewModel(const TWeakObjectPtr<ACompositingElement>& InElement, const TSharedRef<ICompElementManager>& InElementManager)
	: CompElementManager(InElementManager)
	, CommandList(MakeShareable(new FUICommandList))
	, ElementObj(InElement)
{}

void FCompElementViewModel::Initialize()
{
	BindCommands();
}

FName FCompElementViewModel::GetFName() const
{
	if (!ElementObj.IsValid())
	{
		return NAME_None;
	}

	return ElementObj->GetCompElementName();
}


FString FCompElementViewModel::GetName() const
{
	FString Name;
	if (ElementObj.IsValid())
	{
		Name = ElementObj->GetCompElementName().ToString();
	}
	return Name;
}

FText FCompElementViewModel::GetNameAsText() const
{
	if (!ElementObj.IsValid())
	{
		return FText::GetEmpty();
	}

	FText CompName = FText::FromName(ElementObj->GetCompElementName());
	if (UChildActorComponent* ChildActorComp = ElementObj->GetParentComponent())
	{
		CompName = FText::Format(LOCTEXT("ChildActorNameFormat", "{0} (ChildActor)"), CompName);
	}
	return CompName;
}

const TSharedRef< FUICommandList > FCompElementViewModel::GetCommandList() const
{
	return CommandList;
}

bool FCompElementViewModel::IsSetToRender() const
{
	if (ElementObj.IsValid())
	{
		TGuardValue<bool> RunInEditorGuard(ElementObj->bRunInEditor, true);
		return ElementObj->IsActivelyRunning();
	}
	return false;
}

void FCompElementViewModel::ToggleRendering()
{
	if (ElementObj.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("ToggleRendering", "Toggle Element Rendering"));
		CompElementManager->ToggleElementRendering(ElementObj->GetCompElementName());
	}
}

bool FCompElementViewModel::IsRenderingExternallyDisabled() const
{
	if (ElementObj.IsValid())
	{
		TGuardValue<bool> AutoRunGuard(ElementObj->bAutoRun, true);
		TGuardValue<bool> RunInEditorGuard(ElementObj->bRunInEditor, true);

		// when we turn both params above on, this should return true (unless there is some 
		// other state preventing it from happening)
		return !ElementObj->IsActivelyRunning();
	}
	return true;
}

bool FCompElementViewModel::IsEditable() const
{
	if (ElementObj.IsValid())
	{
		UChildActorComponent* ChildActorComp = ElementObj->GetParentComponent();
		return (ChildActorComp == nullptr);
	}
	return false;
}

bool FCompElementViewModel::IsFrameFrozen() const
{
	if (ElementObj.IsValid())
	{
		return ElementObj->FreezeFrameController.HasAllFlags(ETargetUsageFlags::USAGE_Input | ETargetUsageFlags::USAGE_Transform);
	}
	return false;
}

void FCompElementViewModel::ToggleFreezeFrame()
{
	if (ElementObj.IsValid() && !ElementObj->FreezeFrameController.IsLocked())
	{
		const FScopedTransaction Transaction(LOCTEXT("ToggleFreezeFrame", "Toggle Freeze Frame"));
		CompElementManager->ToggleElementFreezeFrame(ElementObj->GetCompElementName());
	}
}

bool FCompElementViewModel::IsFreezeFramingPermitted() const
{
	if (ElementObj.IsValid())
	{
		return !ElementObj->FreezeFrameController.IsLocked() && IsSetToRender();
	}
	return false;
}

bool FCompElementViewModel::HasMediaCaptureSetup(bool& bIsActive) const
{
	bool bFoundExistingOutput = false;
	bIsActive = false;

	if (ElementObj.IsValid())
	{
		UCompositingMediaCaptureOutput* MediaOutput = Cast<UCompositingMediaCaptureOutput>(ElementObj->FindOutputPass(UCompositingMediaCaptureOutput::StaticClass()));
		if (MediaOutput)
		{
			bFoundExistingOutput = true;
			bIsActive = MediaOutput->IsCapturing();
		}
	}

	return bFoundExistingOutput;
}

void FCompElementViewModel::ToggleMediaCapture()
{
	if (ElementObj.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("ToggleMediaCapture", "Toggle Media Output"));
		CompElementManager->ToggleMediaCapture(ElementObj->GetCompElementName());
	}
}

void FCompElementViewModel::RemoveMediaCapture()
{
	if (ElementObj.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveMediaCapture", "Remove Media Output"));
		CompElementManager->RemoveMediaCapture(ElementObj->GetCompElementName());
	}
}

void FCompElementViewModel::ResetMediaCapture()
{
	if (ElementObj.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("ResetMediaCapture", "Reset Media Output"));
		CompElementManager->ResetMediaCapture(ElementObj->GetCompElementName());
	}
}


float FCompElementViewModel::GetElementOpacity() const
{
	if (ElementObj.IsValid())
	{
		return ElementObj->GetOpacity();
	}
	return 0.f;
}

void FCompElementViewModel::SetElementOpacity(const float NewOpacity, const bool bInteractive)
{
	if (ElementObj.IsValid())
	{
		if (bInteractive)
		{
			ElementObj->Modify();
			ElementObj->SetOpacity(NewOpacity);
		}
		else
		{
			const FScopedTransaction Transaction(LOCTEXT("SetElementOpacity", "Set Element Opacity"));
			ElementObj->Modify();
			ElementObj->SetOpacity(NewOpacity);
		}

		CompElementManager->RequestRedraw();
	}
}

bool FCompElementViewModel::IsOpacitySettingEnabled() const
{
	if (ElementObj.IsValid())
	{
		const float RestoreOpacity = ElementObj->GetOpacity();
		if (RestoreOpacity <= 0.f)
		{
			ElementObj->SetOpacity(1.f);
			const bool bDisabledWithoutOpacity = ElementObj->IsActivelyRunning();
			ElementObj->SetOpacity(RestoreOpacity);

			return bDisabledWithoutOpacity;
		}
		return ElementObj->IsActivelyRunning();
	}
	return false;
}

bool FCompElementViewModel::CanRenameTo(const FName& NewCompName, FString& OutMessage) const
{
	if (NewCompName.IsNone())
	{
		OutMessage = LOCTEXT("EmptyCompName", "Comp must be given a name").ToString();
		return false;
	}

	TWeakObjectPtr<ACompositingElement> FoundComp;
	if (CompElementManager->TryGetElement(NewCompName, FoundComp) && FoundComp != ElementObj)
	{
		OutMessage = LOCTEXT("RenameFailed_AlreadyExists", "This comp already exists").ToString();
		return false;
	}

	return true;
}


void FCompElementViewModel::RenameTo(const FName& NewCompName)
{
	if (!ElementObj.IsValid())
	{
		return;
	}

	if (ElementObj->GetCompElementName() == NewCompName)
	{
		return;
	}

	int32 CompIndex = 0;
	FName UniqueNewCompName = NewCompName;
	TWeakObjectPtr<ACompositingElement> FoundComp;
	while (CompElementManager->TryGetElement(UniqueNewCompName, FoundComp))
	{
		UniqueNewCompName = FName(*FString::Printf(TEXT("%s_%d"), *NewCompName.ToString(), ++CompIndex));
	}

	const FScopedTransaction Transaction( LOCTEXT("RenameTo", "Rename Element") );
	CompElementManager->RenameElement(ElementObj->GetCompElementName(), UniqueNewCompName);
}

void FCompElementViewModel::AttachCompElements(const TArray<FName> ElementNames)
{
	if (IsEditable())
	{
		TWeakObjectPtr<ACompositingElement> DataSrc = GetDataSource();
		ULevel* MyLevel = DataSrc->GetLevel();

		FScopedTransaction Transaction(LOCTEXT("UndoReparentElement", "Reparent Element(s)"));
		for (FName DraggedElement : ElementNames)
		{
			TWeakObjectPtr<ACompositingElement> PerspectiveChild;
			if (CompElementManager->TryGetElement(DraggedElement, PerspectiveChild))
			{
				if (PerspectiveChild->GetLevel() != MyLevel)
				{
					continue;
				}
			}

			ACompositingElement* Parent = DataSrc->GetElementParent();
			bool bIsParent = false;
			while (Parent != nullptr)
			{
				if (Parent->GetCompElementName() == DraggedElement)
				{
					// TODO: Should we allow child elements to swap with their parent here?
					bIsParent = true;
					break;
				}
				Parent = Parent->GetElementParent();
			}

			if (!bIsParent && DataSrc->GetCompElementName() != DraggedElement)
			{
				CompElementManager->AttachCompElement(DataSrc->GetCompElementName(), DraggedElement);
			}
		}
	}
}

const TWeakObjectPtr<ACompositingElement> FCompElementViewModel::GetDataSource()
{
	return ElementObj;
}


void FCompElementViewModel::BindCommands()
{
	const FCompElementEditorCommands& Commands = FCompElementEditorCommands::Get();
	FUICommandList& ActionList = *CommandList;

	ActionList.MapAction(Commands.RemoveMediaOutput,
		FExecuteAction::CreateSP(this, &FCompElementViewModel::RemoveMediaCapture),
		FCanExecuteAction::CreateSP(this, &FCompElementViewModel::RemoveMediaCapture_CanExecute));

	ActionList.MapAction(Commands.ResetMediaOutput,
		FExecuteAction::CreateSP(this, &FCompElementViewModel::ResetMediaCapture),
		FCanExecuteAction::CreateSP(this, &FCompElementViewModel::ResetMediaCapture_CanExecute));
}

bool FCompElementViewModel::RemoveMediaCapture_CanExecute() const
{
	bool bIsActive = false;
	return HasMediaCaptureSetup(bIsActive);
}

bool FCompElementViewModel::ResetMediaCapture_CanExecute() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
