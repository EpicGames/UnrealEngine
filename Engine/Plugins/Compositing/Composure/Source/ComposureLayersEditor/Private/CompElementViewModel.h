// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/WeakObjectPtr.h"

class FUICommandList;
class ICompElementManager;

class ACompositingElement;

/**
 * The non-UI solution specific presentation logic for a single comp element.
 */
class FCompElementViewModel : public TSharedFromThis<FCompElementViewModel>
{
public:
	/**  
	 *	Factory method which creates a new FCompElementViewModel object.
	 *
	 *	@param	InElement			The element to wrap
	 *	@param	InElementsManager	The element management logic object
	 */
	static TSharedRef<FCompElementViewModel> Create(const TWeakObjectPtr<ACompositingElement>& InElement, const TSharedRef<ICompElementManager>& InElementsManager)
	{
		TSharedRef<FCompElementViewModel> NewElement(new FCompElementViewModel(InElement, InElementsManager));
		NewElement->Initialize();

		return NewElement;
	}

public:
	/**	Retrieve the element's display name as a FName. */
	FName GetFName() const;
	/**	Retrieve the element's display name as a FString. */
	FString GetName() const;
	/**	Retrieve the element's display name as a FText. */
	FText GetNameAsText() const;

	/** Returns the bound UICommandList for this element row. */
	const TSharedRef<FUICommandList> GetCommandList() const;

	/** Determines if the element is enabled and rendering. */
	bool IsSetToRender() const;
	void ToggleRendering();
	bool IsRenderingExternallyDisabled() const;
	/** Returns whether this element is editable (certain elements, like child actors, are not editable). */
	bool IsEditable() const;

	/** Determines if the element's rendering is paused. */
	bool IsFrameFrozen() const;
	void ToggleFreezeFrame();
	bool IsFreezeFramingPermitted() const;

	/** Determines if the element has a media-capture output pass (and if it's enabled/active). */
	bool HasMediaCaptureSetup(bool& bIsActive) const;
	void ToggleMediaCapture();
	void RemoveMediaCapture();
	void ResetMediaCapture();

	/** Provides a way for users to easily adjust a element's final opacity. */
	float GetElementOpacity() const;
	void  SetElementOpacity(const float NewOpacity, const bool bInteractive = false);
	bool IsOpacitySettingEnabled() const;

	/**
	 *	Returns whether the element can be assigned the specified name.
	 *
	 *	@param	NewElementName		The desired new name
	 *	@param	OutMessage [OUT]	A returned description explaining the boolean result
	 *	@return						True when the name can be assigned.
	 */
	bool CanRenameTo(const FName& NewElementName, FString& OutMessage) const;
	/** Renames the element to the specified name. */
	void RenameTo(const FName& NewElementName);

	/** Child elements, parented to this one. */
	TArray< TSharedPtr<FCompElementViewModel> > Children;
	/** Reparents the specified elements to be children nested under this element. */
	void AttachCompElements(const TArray<FName> ElementNames);

	/** Returns The actor this view-model represents. */
	const TWeakObjectPtr<ACompositingElement> GetDataSource();

	/********************************************************************
	 * EVENTS
	 ********************************************************************/

	/** Broadcasts whenever renaming a element is requested. */
	DECLARE_EVENT( FCompElementViewModel, FRenamedRequestEvent )
	FRenamedRequestEvent& OnRenamedRequest() { return RenamedRequestEvent; }
	void BroadcastRenameRequest() { RenamedRequestEvent.Broadcast(); }

	/** Broadcasts whenever a element preview is requested. */
	DECLARE_EVENT(FCompElementViewModel, FPreviewRequestEvent)
	FPreviewRequestEvent& OnPreviewRequest() { return PreviewRequestEvent; }
	void BroadcastPreviewRequest() { PreviewRequestEvent.Broadcast(); }

private:
	/**
	 *	Private constructor to force users to go through Create(), which properly initializes the model.
	 *
	 *	@param	InElement			The element to wrap
	 *	@param	InElementsManager	The element management logic object
	 */
	FCompElementViewModel(const TWeakObjectPtr<ACompositingElement>& InElement, const TSharedRef<ICompElementManager>& InElementsManager);

	/** Initializes the FCompElementViewModel for use */
	void Initialize();
	/** Binds individual element row commands to delegates. */
	void BindCommands();

	/** Removes any media-capture output pass currently applied to this element. */
	bool RemoveMediaCapture_CanExecute() const;
	bool ResetMediaCapture_CanExecute() const;

private:
	/** The element management logic object. */
	const TSharedRef<ICompElementManager> CompElementManager;
	/** The list of commands with bound delegates for this specific element. */
	const TSharedRef<FUICommandList> CommandList;

	/** The element actor this object represents. */
	TWeakObjectPtr<ACompositingElement> ElementObj;

	/** Broadcasts whenever a rename is requested. */
	FRenamedRequestEvent RenamedRequestEvent;
	/** Broadcasts whenever a element preview is requested. */
	FPreviewRequestEvent PreviewRequestEvent;
};
