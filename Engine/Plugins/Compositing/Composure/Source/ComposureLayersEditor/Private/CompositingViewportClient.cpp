// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositingViewportClient.h"
#include "CompositingElement.h"
#include "Editor/UnrealEdTypes.h"
#include "EditorCompElementContainer.h"
#include "LevelEditorViewport.h" // for GetIsCameraCut()

static TAutoConsoleVariable<int32> CVarDecoupleEditorCompRendering(
	TEXT("r.Composure.CompositingElements.Editor.DecoupleRenderingFromLevelViewport"),
	1,
	TEXT("In editor, this decouples the compositing rendering from the editor's level rendering (to not be limited by the ")
	TEXT("on-demand rendering style it sometimes uses). It uses a dedicated (hidden) viewport to enqueue the compositing render commands."));

static TAutoConsoleVariable<int32> CVarCompositingRealtimeRendering(
	TEXT("r.Composure.CompositingElements.Editor.RealtimeRendering"),
	1,
	TEXT("Turns on/off the realtime compositing rendering done by the dedicated compositing viewport."));


DEFINE_LOG_CATEGORY_STATIC(LogComposureCompositingEditor, Log, All)

/* FCompositingViewport
 *****************************************************************************/

FCompositingViewport::FCompositingViewport()
	: FDummyViewport()
{
	// Need a non-zero size to call into FEditorViewportClient::Draw()
	SizeX = 1920;
	SizeY = 1080;
}

TSharedRef<FCompositingViewport> FCompositingViewport::Create(TWeakObjectPtr<UEditorCompElementContainer>&& CompElements)
{
	TSharedRef<FCompositingViewportClient> Client = MakeShared<FCompositingViewportClient>(MoveTemp(CompElements));
	TSharedRef<FCompositingViewport> Result = MakeShareable(new FCompositingViewport());
	Result->SetViewportClient(Client);
	Client->Viewport = &Result.Get();
	return Result;
}

FCompositingViewportClient& FCompositingViewport::GetCompositingClient() const
{
	return static_cast<FCompositingViewportClient&>(GetClientChecked());
}

bool FCompositingViewport::IsDrawing() const
{
	return GetCompositingClient().IsDrawing();
}

void FCompositingViewport::RequestRedraw()
{
	GetCompositingClient().RedrawRequested(this);
}

/* FCompositingViewportClient
 *****************************************************************************/

FCompositingViewportClient::FCompositingViewportClient(TWeakObjectPtr<UEditorCompElementContainer>&& CompElements)
	: FEditorViewportClient(nullptr)
	, ElementsContainerPtr(MoveTemp(CompElements))
{
	SetViewModes(VMI_Unlit, VMI_Unlit);
	SetViewportType(LVT_OrthoFreelook);

	//SetRealtime(true);
	VisibilityDelegate.BindRaw(this, &FCompositingViewportClient::InternalIsVisible);
}

FCompositingViewportClient::~FCompositingViewportClient()
{
	Viewport = nullptr;
}

bool FCompositingViewportClient::IsDrawing() const
{
	return bIsDrawing;
}

void FCompositingViewportClient::Draw(const FSceneView* /*View*/, FPrimitiveDrawInterface* /*PDI*/)
{
	// DO NOTHING
}

void FCompositingViewportClient::Draw(FViewport* /*InViewport*/, FCanvas* /*Canvas*/)
{
	if (ElementsContainerPtr.IsValid())
	{
		bIsDrawing = true;

		struct FCompElementDrawOrderSort
		{
			// Should Lhs come before Rhs?
			FORCEINLINE bool operator()(const TWeakObjectPtr<ACompositingElement>& Lhs, const TWeakObjectPtr<ACompositingElement>& Rhs) const
			{
				if (Lhs.IsValid() && Rhs.IsValid())
				{
					return Lhs->GetRenderPriority() > Rhs->GetRenderPriority();
				}
				return !Lhs.IsValid();
			}
		};
		ElementsContainerPtr->Sort(FCompElementDrawOrderSort());

		bool bCameraCut = false;
		for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
		{
			bCameraCut = LevelVC->GetIsCameraCut();
			if (bCameraCut)
			{
				break;
			}
		}

		for (const TWeakObjectPtr<ACompositingElement>& ElementPtr : *ElementsContainerPtr.Get())
		{
			if (ElementPtr.IsValid())
			{
				ACompositingElement* Element = ElementPtr.Get();
				if (Element->IsActivelyRunning())
				{
					Element->EnqueueRendering(bCameraCut);
				}
			}
			else
			{
				break;
			}
		}

		bIsDrawing = false;
	}
}

void FCompositingViewportClient::DrawCanvas(FViewport& /*InViewport*/, FSceneView& /*View*/, FCanvas& /*Canvas*/)
{
	// DO NOTHING
}

bool FCompositingViewportClient::ProcessScreenShots(FViewport* /*InViewport*/)
{
	// DO NOTHING
	return false;
}

bool FCompositingViewportClient::WantsDrawWhenAppIsHidden() const
{
	return !!CVarDecoupleEditorCompRendering.GetValueOnGameThread() && (IsRealtime() || bNeedsRedraw);
}

void FCompositingViewportClient::Tick(float DeltaSeconds)
{
	// Since "Realtime" rendered viewports could still get throttled by in-editor events,
	// we need a better way to ensure our Draw() happens. So each frame we manually mark 
	// ourselves as needing a re-draw (which is not throttled).
	if (CVarCompositingRealtimeRendering.GetValueOnGameThread() > 0)
	{
		RedrawRequested(Viewport);
	}
}

bool FCompositingViewportClient::IsTickable() const
{
	return !!CVarCompositingRealtimeRendering.GetValueOnGameThread();
}

TStatId FCompositingViewportClient::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCompositingViewportClient, STATGROUP_Tickables);
}

bool FCompositingViewportClient::InternalIsVisible() const
{
	return !!CVarDecoupleEditorCompRendering.GetValueOnGameThread() && (IsRealtime() || bNeedsRedraw);
}
