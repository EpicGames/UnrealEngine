// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DummyViewport.h"
#include "EditorViewportClient.h"
#include "TickableEditorObject.h"

class FCanvas;
class UEditorCompElementContainer;

class FCompositingViewportClient final : public FEditorViewportClient, public FTickableEditorObject
{
public:
	FCompositingViewportClient(TWeakObjectPtr<UEditorCompElementContainer>&& CompElements);
	virtual ~FCompositingViewportClient();

	bool IsDrawing() const;

	//~ Begin FViewportClient interface
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;
	virtual bool ProcessScreenShots(FViewport* Viewport) override;
	//~ End FViewportClient interface

	//~ Begin FEditorViewportClient interface
	virtual bool WantsDrawWhenAppIsHidden() const override;
	//~ End FEditorViewportClient Interface

	//~ Begin FTickableEditorObject interface
	virtual void Tick(float DeltaSeconds) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject interface

private:
	bool InternalIsVisible() const;

	TWeakObjectPtr<UEditorCompElementContainer> ElementsContainerPtr;
	bool bIsDrawing = false;
};

class FCompositingViewport final : public FDummyViewport
{
	FCompositingViewport();
public:
	static TSharedRef<FCompositingViewport> Create(TWeakObjectPtr<UEditorCompElementContainer>&& CompElements);

	FCompositingViewportClient& GetCompositingClient() const;
	bool IsDrawing() const;
	void RequestRedraw();
};
