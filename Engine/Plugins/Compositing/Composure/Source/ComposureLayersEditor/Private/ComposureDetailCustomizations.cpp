// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureDetailCustomizations.h"
#include "CompositingElements/CompositingMaterialPass.h"
#include "DetailLayoutBuilder.h"
#include "Customizations/MathStructCustomizations.h"
#include "IDetailChildrenBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "IPropertyUtilities.h"
#include "Customizations/ColorStructCustomization.h"
#include "ICompElementManager.h"
#include "Widgets/Input/SButton.h"
#include "Materials/MaterialInterface.h"
#include "Widgets/SToolTip.h"
#include "ComposureEditorStyle.h"
#include "Widgets/SCompElementPickerWindow.h"
#include "CompositingElement.h"
#include "IDetailGroup.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Editor.h"
#include "CompositingElements/CompositingElementPasses.h"
#include "Modules/ModuleManager.h"
#include "CompElementEditorModule.h"
#include "ScopedTransaction.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "ComposureDetailCustomizations"

/* FComposureColorPickerCustomization
*****************************************************************************/

class FComposureColorPickerCustomization : public FColorStructCustomization
{
public:
	FComposureColorPickerCustomization(TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget);

public:
	//~ FMathStructCustomization interface
	virtual void MakeHeaderRow(TSharedRef<IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row) override;

private:
	FReply OnOpenPickerClick(TSharedRef<IPropertyHandle> PropertyHandle);
	void OnColorSelected(const FVector2D& /*PickedUV*/, const FLinearColor& PickedColor, bool bInteractive, TSharedRef<IPropertyHandle> PropertyHandle);
	void OnColorReset(TSharedRef<IPropertyHandle> PropertyHandle);

private:
	TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget;
	FString DefaultColorStr;

	bool bIsInteractive = false;
};

FComposureColorPickerCustomization::FComposureColorPickerCustomization(TWeakUIntrfacePtr<ICompImageColorPickerInterface> InPickerTarget)
	: PickerTarget(InPickerTarget)
{}

void FComposureColorPickerCustomization::MakeHeaderRow(TSharedRef<IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row)
{
	Row.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		.MaxDesiredWidth(250.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				CreateColorWidget(StructPropertyHandle)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &FComposureColorPickerCustomization::OnOpenPickerClick, InStructPropertyHandle)
				//.ToolTipText( InArgs._Text )
				.ContentPadding(4.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(FComposureEditorStyle::Get().GetBrush("ComposureProperties.Button_ChromaPicker"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
}

FReply FComposureColorPickerCustomization::OnOpenPickerClick(TSharedRef<IPropertyHandle> PropertyHandle)
{
	FCompElementColorPickerArgs PickerArgs;
	PickerArgs.PickerTarget = PickerTarget;
	PickerArgs.OnColorPicked = FColorPickedEventHandler::CreateSP(this, &FComposureColorPickerCustomization::OnColorSelected, PropertyHandle);
	PickerArgs.OnColorPickerCanceled = FSimpleDelegate::CreateSP(this, &FComposureColorPickerCustomization::OnColorReset, PropertyHandle);
	PickerArgs.ParentWidget = ColorPickerParentWidget;

	TArray<UObject*> OutersList;
	PropertyHandle->GetOuterObjects(OutersList);
	if (OutersList.Num() == 1)
	{
		UObject* Outer = OutersList[0];
		if (Outer)
		{
			FString ObjPathName;
			while (Outer)
			{
				FString OuterStrName;
				if (UCompositingElementPass* AsPass = Cast<UCompositingElementPass>(Outer))
				{
					OuterStrName = AsPass->PassName.ToString();
				}
				else if (ACompositingElement* AsCompShot = Cast<ACompositingElement>(Outer))
				{
					OuterStrName = AsCompShot->GetCompElementName().ToString();
				}
				else
				{
					break;
				}

				if (!ObjPathName.IsEmpty())
				{
					OuterStrName += TEXT(".");
				}
				ObjPathName = OuterStrName + ObjPathName;

				Outer = Outer->GetOuter();
			}

			if (!ObjPathName.IsEmpty())
			{
				PickerArgs.WindowTitle = FText::Format(LOCTEXT("PickerWindowTitle", "Color Picker ({0})"), FText::FromString(ObjPathName));
			}
		}
	}

	SCompElementPickerWindow::Open(PickerArgs);

	DefaultColorStr.Empty(DefaultColorStr.Len());
	PropertyHandle->GetValueAsFormattedString(DefaultColorStr);

	return FReply::Handled();
}

void FComposureColorPickerCustomization::OnColorSelected(const FVector2D& /*PickedUV*/, const FLinearColor& PickedColor, bool bInteractive, TSharedRef<IPropertyHandle> PropertyHandle)
{
	if (bInteractive != bIsInteractive)
	{
		if (bInteractive)
		{
			GEditor->BeginTransaction(LOCTEXT("PickPlateColorTransaction", "Pick Plate Color"));
		}
		else
		{
			GEditor->EndTransaction();
		}
		bIsInteractive = bInteractive;
	}

	PropertyHandle->SetValueFromFormattedString(PickedColor.ToString(), bInteractive ? EPropertyValueSetFlags::InteractiveChange : 0);
	PropertyHandle->NotifyFinishedChangingProperties();
}

void FComposureColorPickerCustomization::OnColorReset(TSharedRef<IPropertyHandle> PropertyHandle)
{
	if (!DefaultColorStr.IsEmpty())
	{
		PropertyHandle->SetValueFromFormattedString(DefaultColorStr);
		PropertyHandle->NotifyFinishedChangingProperties();
	}

	if (bIsInteractive)
	{
		GEditor->EndTransaction();
		bIsInteractive = false;
	}
}

/* FCompElementDetailsCustomization
 *****************************************************************************/

namespace ElementDetailsCustomization_Impl
{
	static bool NeedsCameraSource(ACompositingElement* Element);
}

static bool ElementDetailsCustomization_Impl::NeedsCameraSource(ACompositingElement* Element)
{
	if (Element->CameraSource != ESceneCameraLinkType::Unused)
	{
		return true;
	}
	else
	{
		for (ACompositingElement* Child : Element->GetChildElements())
		{
			if (NeedsCameraSource(Child))
			{
				return true;
			}
		}
	}
	return false;
}

TSharedRef<IDetailCustomization> FCompElementDetailsCustomization::MakeInstance()
{
	return MakeShared<FCompElementDetailsCustomization>();
}

void FCompElementDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	MyLayout = &DetailBuilder;

	TArray< TWeakObjectPtr<UObject> > SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);

	if (SelectedObjects.Num() > 0)
	{
		TWeakObjectPtr<UObject> ObjPtr = SelectedObjects[0];
		if (ObjPtr.IsValid())
		{
			UObject* SelectedObj = ObjPtr.Get();
			if (ACompositingElement* AsElement = Cast<ACompositingElement>(SelectedObj))
			{
				TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget(AsElement);
				FOnGetPropertyTypeCustomizationInstance CustomColorPickerFactory = FOnGetPropertyTypeCustomizationInstance::CreateLambda(
					[PickerTarget]
				{
					return MakeShared<FComposureColorPickerCustomization>(PickerTarget);
				});
				DetailBuilder.RegisterInstancedCustomPropertyTypeLayout("LinearColor", CustomColorPickerFactory);
			}
		}
		
#define TRINARY_TRUE  +1
#define TRINARY_FALSE -1
#define TRINARY_IS_FALSE(Value) (Value < 0)
#define TRINARY_IS_UNSET(Value) (Value == 0)
		// 0 == unset, >0 == true, <0 == false
		int8 bShowResolution = 0; 
		int8 bShowResSource  = 0; 
		int8 bShowCameraProp = 0;
		int8 bShowCamSource  = 0;
		int8 bShowPreviewPass = 0;
		int8 bShowPreviewSrc = 0;

		bool bArchetypeSelected = false;
		for (TWeakObjectPtr<UObject> SelectedObj : SelectedObjects)
		{
			if (ACompositingElement* AsElement = Cast<ACompositingElement>(SelectedObj))
			{
				const bool bIsArchetype = AsElement->IsTemplate();
				bArchetypeSelected |= bIsArchetype;

				ACompositingElement* Parent = AsElement->GetElementParent();
				if (Parent)
				{
					bShowResSource = TRINARY_TRUE;
					bShowPreviewSrc = TRINARY_TRUE;
				}
				else
				{
					if (TRINARY_IS_UNSET(bShowResSource))
					{
						bShowResSource = TRINARY_FALSE;
					}

					if (TRINARY_IS_UNSET(bShowPreviewSrc))
					{
						bShowPreviewSrc = TRINARY_FALSE;
					}
				}

				if (AsElement->ResolutionSource == EInheritedSourceType::Override || !Parent)
				{
					bShowResolution = TRINARY_TRUE;
				}
				else if (TRINARY_IS_UNSET(bShowResolution))
				{
					bShowResolution = TRINARY_FALSE;
				}

				if (AsElement->PreviewTransformSource == EInheritedSourceType::Override || !Parent)
				{
					bShowPreviewPass = TRINARY_TRUE;
				}
				else if (TRINARY_IS_UNSET(bShowPreviewPass))
				{
					bShowPreviewPass = TRINARY_FALSE;
				}


				if ( bIsArchetype || (Parent && ElementDetailsCustomization_Impl::NeedsCameraSource(AsElement)) )
				{
					bShowCamSource = TRINARY_TRUE;
				}
				else if (TRINARY_IS_UNSET(bShowCamSource))
				{
					bShowCamSource = TRINARY_FALSE;
				}

				if ( AsElement->CameraSource == ESceneCameraLinkType::Override || (!Parent && ElementDetailsCustomization_Impl::NeedsCameraSource(AsElement)) )
				{
					bShowCameraProp = TRINARY_TRUE;
				}
				else if (TRINARY_IS_UNSET(bShowCameraProp))
				{
					bShowCameraProp = TRINARY_FALSE;
				}
			}
		}

		TSharedPtr<IPropertyHandle> ResolutionSource = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACompositingElement, ResolutionSource));
		ResolutionSource->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCompElementDetailsCustomization::ForceRefreshLayout));

		if (TRINARY_IS_FALSE(bShowResSource))
		{
 			DetailBuilder.HideProperty(ResolutionSource);
		}
		if (TRINARY_IS_FALSE(bShowResolution))
		{
			DetailBuilder.HideProperty(TEXT("RenderResolution"));//GET_MEMBER_NAME_CHECKED(ACompositingElement, RenderResolution));
		}

		TSharedPtr<IPropertyHandle> CameraSource = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACompositingElement, CameraSource));
		CameraSource->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCompElementDetailsCustomization::ForceRefreshLayout));

		if (TRINARY_IS_FALSE(bShowCamSource))
		{
			DetailBuilder.HideProperty(CameraSource);
		}
		else if (!bArchetypeSelected)
		{
			IDetailPropertyRow* CameraSourceRow = DetailBuilder.EditDefaultProperty(CameraSource);
			if (CameraSourceRow)
			{
				CameraSourceRow->CustomWidget()
					.NameContent()
					[
						CameraSource->CreatePropertyNameWidget()
					]
					.ValueContent()
						.MinDesiredWidth(125.f)
						.MaxDesiredWidth(400.f)
					[
						PropertyCustomizationHelpers::MakePropertyComboBox( 
							CameraSource, 
							FOnGetPropertyComboBoxStrings::CreateSP(this, &FCompElementDetailsCustomization::GetInstanceCameraSourceComboStrings),
							FOnGetPropertyComboBoxValue::CreateSP(this, &FCompElementDetailsCustomization::GetInstanceCameraSourceValueStr, CameraSource),
							FOnPropertyComboBoxValueSelected::CreateSP(this, &FCompElementDetailsCustomization::OnCameraSourceSelected, CameraSource)
						)
					];
			}				
		}

		if (TRINARY_IS_FALSE(bShowCameraProp))
		{
			DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(ACompositingElement, TargetCameraActorPtr));
		}

		TSharedPtr<IPropertyHandle> PreviewTransformSource = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ACompositingElement, PreviewTransformSource));
		PreviewTransformSource->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCompElementDetailsCustomization::ForceRefreshLayout));

		if (TRINARY_IS_FALSE(bShowPreviewSrc))
		{
			DetailBuilder.HideProperty(PreviewTransformSource);
		}
		if (TRINARY_IS_FALSE(bShowPreviewPass))
		{
			DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(ACompositingElement, PreviewTransform));
		}

#undef TRINARY_TRUE
#undef TRINARY_FALSE
#undef TRINARY_IS_FALSE
#undef TRINARY_IS_UNSET
	}
}

void FCompElementDetailsCustomization::ForceRefreshLayout()
{
	if (MyLayout)
	{
		MyLayout->ForceRefreshDetails();
	}
}

void FCompElementDetailsCustomization::GetInstanceCameraSourceComboStrings(TArray<TSharedPtr<FString>>& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems)
{
	const UEnum* CamSourceEnum = StaticEnum<ESceneCameraLinkType>();
	if (ensure(CamSourceEnum))
	{
		for (int32 EnumIndex = 0; EnumIndex < CamSourceEnum->NumEnums()-1; ++EnumIndex)
		{
			if (CamSourceEnum->GetValueByIndex(EnumIndex) != (int64)ESceneCameraLinkType::Unused)
			{
				FText EnumeratorName = CamSourceEnum->GetDisplayNameTextByIndex(EnumIndex);

				OutComboBoxStrings.Add(MakeShared<FString>(EnumeratorName.ToString()));
				OutToolTips.Add(SNew(SToolTip).Text(EnumeratorName));
				OutRestrictedItems.Add(false);
			}
		}
	}	
}

FString FCompElementDetailsCustomization::GetInstanceCameraSourceValueStr(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FString DisplayStr;

	uint8 CurrentValue = (uint8)ESceneCameraLinkType::Inherited;
	FPropertyAccess::Result GetValResult = PropertyHandle.IsValid() ? PropertyHandle->GetValue(CurrentValue) : FPropertyAccess::Fail;

	if (GetValResult == FPropertyAccess::MultipleValues)
	{
		DisplayStr = LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}
	else if (GetValResult == FPropertyAccess::Success)
	{
		if (CurrentValue == (uint8)ESceneCameraLinkType::Unused)
		{
			DisplayStr = LOCTEXT("UnusedDisplayString", "Inherited (Unused/Passthrough)").ToString();
		}
		else
		{
			const UEnum* CamSourceEnum = StaticEnum<ESceneCameraLinkType>();
			if (ensure(CamSourceEnum))
			{
				DisplayStr = CamSourceEnum->GetDisplayNameTextByValue(CurrentValue).ToString();
			}
		}		
	}

	return DisplayStr;
}

void FCompElementDetailsCustomization::OnCameraSourceSelected(const FString& Selection, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	const UEnum* CamSourceEnum = StaticEnum<ESceneCameraLinkType>();
	if (ensure(CamSourceEnum) && PropertyHandle.IsValid())
	{
		const int64 FoundValue = CamSourceEnum->GetValueByNameString(Selection);
		if (FoundValue == (int64)ESceneCameraLinkType::Override)
		{
			PropertyHandle->SetValue((uint8)FoundValue);
		}
		else if (FoundValue == (int64)ESceneCameraLinkType::Inherited)
		{
			TArray<UObject*> OuterObjects;
			PropertyHandle->GetOuterObjects(OuterObjects);

			bool bSetToUnused = true;
			for (UObject* Outer : OuterObjects)
			{
				if (Outer)
				{
					UClass* ObjClass = Outer->GetClass();
					const ACompositingElement* CDO = ObjClass ? Cast<ACompositingElement>(ObjClass->GetDefaultObject(false)): nullptr;

					if (CDO && CDO->CameraSource != ESceneCameraLinkType::Unused)
					{
						bSetToUnused = false;
						break;
					}
				}
			}

			if (bSetToUnused)
			{
				PropertyHandle->SetValue((uint8)ESceneCameraLinkType::Unused);
			}
			else
			{
				PropertyHandle->SetValue((uint8)ESceneCameraLinkType::Inherited);
			}
		}
	}
}

/* FCompositingMaterialPassCustomization
*****************************************************************************/

FCompositingMaterialPassCustomization::FCompositingMaterialPassCustomization()
{
	FEditorDelegates::RefreshEditor.AddRaw(this, &FCompositingMaterialPassCustomization::OnRedrawViewports);
	GEditor->RegisterForUndo(this);
}

FCompositingMaterialPassCustomization::~FCompositingMaterialPassCustomization()
{
	FEditorDelegates::RefreshEditor.RemoveAll(this);
	GEditor->UnregisterForUndo(this);
}

TSharedRef<IPropertyTypeCustomization> FCompositingMaterialPassCustomization::MakeInstance()
{
	return MakeShareable(new FCompositingMaterialPassCustomization);
}

void FCompositingMaterialPassCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	CachedPropertyHandle = PropertyHandle;
	CachedUtils = CustomizationUtils.GetPropertyUtilities();

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		UObject* Obj = OuterObjects[0];
		if (Obj->IsA<UCompositingElementPass>())
		{
			MaterialPassName = &CastChecked<UCompositingElementPass>(Obj)->PassName;
		}
	}

	FCompositingMaterialType* MatPass = GetMaterialPass();
	check(MatPass);
	MaterialReference = TWeakObjectPtr<UMaterialInterface>(MatPass->Material);

	ICompElementEditorModule& ComposureLayersModule = FModuleManager::GetModuleChecked<ICompElementEditorModule>(TEXT("ComposureLayersEditor"));
	CompElementManager = ComposureLayersModule.GetCompElementManager();
}

void FCompositingMaterialPassCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	check(CachedUtils);
	check(CachedPropertyHandle == StructPropertyHandle);

	FSimpleDelegate Refresh = FSimpleDelegate::CreateRaw(CachedUtils.Get(), &IPropertyUtilities::ForceRefresh);
	FSimpleDelegate ResetOverrides = FSimpleDelegate::CreateSP(this, &FCompositingMaterialPassCustomization::ResetParameterOverrides);

	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();

		if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FCompositingMaterialType, Material))
		{
			MaterialHandle = ChildHandle;
			ChildHandle->SetOnPropertyValueChanged(ResetOverrides);
		}

		else if (ChildHandle->GetProperty()->GetFName() == FName(TEXT("VectorOverrideProxies")))//GET_MEMBER_NAME_CHECKED(FCompositingMaterialPass, VectorOverrideProxies))
		{
			CachedVectorProxies = ChildHandle;
			continue;
		}

		else if (ChildHandle->GetProperty()->GetFName() == FName(TEXT("ParamPassMappings")))//GET_MEMBER_NAME_CHECKED(FCompositingMaterialPass, MaterialParamMappings))
		{
			CachedMaterialParamMappings = ChildHandle;
			continue;
		}

		IDetailPropertyRow& Property = StructBuilder.AddProperty(ChildHandle);
	}

	if (UMaterialInterface* MaterialReferencePtr = MaterialReference.Get())
	{
		FCompositingMaterialType* MatPass = GetMaterialPass();
		check(MatPass);

		//Required material params
		if (MatPass->RequiredMaterialParams.Num() > 0)
		{
			IDetailGroup& RequiredParamsGroup = StructBuilder.AddGroup(FName(TEXT("RequiredParamsGroup")), LOCTEXT("RequiredParamsGroup_DisplayName", "Required Material Parameters"));

			for (TPair<FName, FNamedCompMaterialParam> RequiredParam : MatPass->RequiredMaterialParams)
			{
				const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
				TSharedPtr<SComboButton> RequiredParamComboButton = SNew(SComboButton)
					.ContentPadding(FMargin(0, 0, 5, 0))
					.ButtonContent()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("NoBorder"))
						.Padding(FMargin(0, 0, 5, 0))
						[
							SNew(SEditableTextBox)
							.Text(this, &FCompositingMaterialPassCustomization::GetRequiredParamComboText, RequiredParam.Key)
							.OnTextCommitted(this, &FCompositingMaterialPassCustomization::OnRequiredParamComboTextCommitted, RequiredParam.Key)
							.SelectAllTextWhenFocused(true)
							.RevertTextOnEscape(true)
							.Font(DetailFontInfo)
						]
					];

				TWeakPtr<SComboButton> WeakParamComboPtr = RequiredParamComboButton;
				RequiredParamComboButton->SetOnGetMenuContent(FOnGetContent::CreateSP(this, &FCompositingMaterialPassCustomization::GetRequiredParamComboMenu, WeakParamComboPtr, RequiredParam.Key, RequiredParam.Value.ParamType));

				// Populate dropdowns
				RebuildRequiredParamSources();

				RequiredParamsGroup.AddWidgetRow()
					.NameContent()
					[
						SNew(SBox)
						.Padding(FMargin(15, 0))
						.Content()
						[
							SNew(STextBlock)
							.Text(FText::FromName(RequiredParam.Key))
							.Font(DetailFontInfo)
						]
					]
					.ValueContent()
					.MinDesiredWidth(166.0f)
					[
						RequiredParamComboButton.ToSharedRef()
					];
			}
		}

		TArray<FMaterialParameterInfo> OutScalarParameterInfo;
		TArray<FGuid> ScalarGuids;
		MaterialReferencePtr->GetAllScalarParameterInfo(OutScalarParameterInfo, ScalarGuids);

		TArray<FMaterialParameterInfo> OutVectorParameterInfo;
		TArray<FGuid> VectorGuids;
		MaterialReferencePtr->GetAllVectorParameterInfo(OutVectorParameterInfo, VectorGuids);

		TArray<FMaterialParameterInfo> OutTextureParameterInfo;
		TArray<FGuid> TextureGuids;
		MaterialReferencePtr->GetAllTextureParameterInfo(OutTextureParameterInfo, TextureGuids);

		if (OutTextureParameterInfo.Num() > 0)
		{
			uint32 TexChildren = 0;
			CachedMaterialParamMappings->GetNumChildren(TexChildren);

			if (TexChildren > 0)
			{
				IDetailGroup& InputElementsGroup = StructBuilder.AddGroup(FName(TEXT("InputElementsGroup")), LOCTEXT("InputElementsGroup_DisplayName", "Input Elements"));

				//Texture Params

				for (uint32 ChildIndex = 0; ChildIndex < TexChildren; ++ChildIndex)
				{

					TSharedPtr<IPropertyHandle> ChildHandle = CachedMaterialParamMappings->GetChildHandle(ChildIndex);
					FName TextureName;
					ChildHandle->GetKeyHandle()->GetValue(TextureName);

					bool IsRequiredParam = false;

					for (TPair<FName, FNamedCompMaterialParam> Param : MatPass->RequiredMaterialParams)
					{
						if ((Param.Value.ParamType == EParamType::TextureParam || Param.Value.ParamType == EParamType::UnknownParamType) && Param.Value.ParamName == TextureName)
						{
							//Texture is in use by a required param, so hide it
							IsRequiredParam = true;
						}
					}

					TSharedPtr<SWidget> NameWidget;
					TSharedPtr<SWidget> ValueWidget;
					FDetailWidgetRow Row;

					IDetailPropertyRow& PropertyRow = InputElementsGroup.AddPropertyRow(ChildHandle.ToSharedRef());

					const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
					TSharedPtr<SComboButton> PassComboButton = SNew(SComboButton)
						.ContentPadding(FMargin(0, 0, 5, 0))
						//.IsEnabled(this, &FBlueprintVarActionDetails::GetVariableCategoryChangeEnabled)
						//.ToolTip(CategoryTooltip)
						.ButtonContent()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("NoBorder"))
							.Padding(FMargin(0, 0, 5, 0))
							[
								SNew(SEditableTextBox)
								.Text(this, &FCompositingMaterialPassCustomization::GetComboText, TextureName)
								.OnTextCommitted(this, &FCompositingMaterialPassCustomization::OnComboTextCommitted, TextureName)
								//.ToolTip(CategoryTooltip)
								.SelectAllTextWhenFocused(true)
								.RevertTextOnEscape(true)
								.Font(DetailFontInfo)
							]
						];

					TWeakPtr<SComboButton> WeakComboPtr = PassComboButton;
					PassComboButton->SetOnGetMenuContent(FOnGetContent::CreateSP(this, &FCompositingMaterialPassCustomization::GetPassComboMenu, WeakComboPtr, TextureName));

					RebuildTextureSourceList();

					FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FCompositingMaterialPassCustomization::TextureShouldShowResetToDefault);
					FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FCompositingMaterialPassCustomization::TextureResetToDefault, PassComboButton);
					FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);

					//ChildHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCompositingMaterialPassCustomization::OnTextureOverrideChanged, ChildHandle));

					PropertyRow.OverrideResetToDefault(ResetOverride);
					PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
					PropertyRow
						.ShowPropertyButtons(false)
						.IsEnabled(!IsRequiredParam)
						.CustomWidget()
						.NameContent()
						[
							SNew(SBox)
							.Padding(FMargin(15, 0))
							.Content()
							[
								SNew(STextBlock)
								.Text(FText::FromString(TextureName.ToString()))
								.Font(DetailFontInfo)
							]
						]
						.ValueContent()
						.MinDesiredWidth(166.0f)
						[
							PassComboButton.ToSharedRef()
						];
				}
			}
		}

		if (OutScalarParameterInfo.Num() + OutVectorParameterInfo.Num() > 0)
		{
			IDetailGroup& MaterialParametersGroup = StructBuilder.AddGroup(FName(TEXT("MaterialParametersGroup")), LOCTEXT("MaterialParametersGroup_DisplayName", "Material Parameters"));

			//Scalar Params

			for (FMaterialParameterInfo ScalarParam : OutScalarParameterInfo)
			{
				if (MatPass->EditorHiddenParams.Contains(ScalarParam.Name))
				{
					continue;
				}

				bool IsRequiredParam = false;

				for (TPair<FName, FNamedCompMaterialParam> Param : MatPass->RequiredMaterialParams)
				{
					if ((Param.Value.ParamType == EParamType::ScalarParam || Param.Value.ParamType == EParamType::UnknownParamType) && Param.Value.ParamName == ScalarParam.Name)
					{
						//Scalar is in use by a required param, so hide it
						IsRequiredParam = true;
					}
				}

				MaterialParametersGroup.AddWidgetRow()
					.NameContent()
					[
						SNew(SBox)
						.Padding(FMargin(15, 0))
						.IsEnabled(!IsRequiredParam)
						.Content()
						[
							SNew(STextBlock)
							.IsEnabled(!IsRequiredParam)
							.Text(FText::FromString(ScalarParam.Name.ToString()))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
					.ValueContent()
					[
						SNew(SHorizontalBox)
						.IsEnabled(!IsRequiredParam)
						+ SHorizontalBox::Slot()
						.Padding(0.0f, 0.0f, 4.0f, 0.0f)
						[
							SNew(SNumericEntryBox<float>)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.AllowSpin(true)
							.MinValue(TOptional<float>())
							.MaxValue(TOptional<float>())
							.MinSliderValue(this, &FCompositingMaterialPassCustomization::GetScalarParameterSliderMin, ScalarParam)
							.MaxSliderValue(this, &FCompositingMaterialPassCustomization::GetScalarParameterSliderMax, ScalarParam)
							.Delta(0.0f)
							.Value(this, &FCompositingMaterialPassCustomization::GetScalarParameterValue, ScalarParam)
							.OnBeginSliderMovement(this, &FCompositingMaterialPassCustomization::OnScalarParameterSlideBegin, ScalarParam)
							.OnEndSliderMovement(this, &FCompositingMaterialPassCustomization::OnScalarParameterSlideEnd, ScalarParam)
							.OnValueChanged(this, &FCompositingMaterialPassCustomization::SetScalarParameterValue, ScalarParam)
							.OnValueCommitted(this, &FCompositingMaterialPassCustomization::OnScalarParameterCommitted, ScalarParam)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2, 1)
						[
							SNew(SButton)
							.IsFocusable(false)
							.ToolTipText(LOCTEXT("ResetToDefaultToolTip", "Reset to Default"))
							.ButtonStyle(FAppStyle::Get(), "NoBorder")
							.ContentPadding(0)
							.Visibility(this, &FCompositingMaterialPassCustomization::IsResetScalarParameterVisible, ScalarParam)
							.OnClicked(this, &FCompositingMaterialPassCustomization::OnResetScalarParameterClicked, ScalarParam)
							.Content()
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
							]
						]
					];
			}

			//Vector Params

			uint32 children = 0;
			CachedVectorProxies->GetNumChildren(children);

			for (uint32 ChildIndex = 0; ChildIndex < children; ++ChildIndex)
			{

				TSharedPtr<IPropertyHandle> ChildHandle = CachedVectorProxies->GetChildHandle(ChildIndex);
				FName VectorName;
				ChildHandle->GetKeyHandle()->GetValue(VectorName);

				if (MatPass->EditorHiddenParams.Contains(VectorName))
				{
					continue;
				}

				bool IsRequiredParam = false;

				for (TPair<FName, FNamedCompMaterialParam> Param : MatPass->RequiredMaterialParams)
				{
					if ((Param.Value.ParamType == EParamType::VectorParam || Param.Value.ParamType == EParamType::UnknownParamType) && Param.Value.ParamName == VectorName)
					{
						//Texture is in use by a required param, so hide it
						IsRequiredParam = true;
					}
				}

				FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FCompositingMaterialPassCustomization::VectorShouldShowResetToDefault);
				FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FCompositingMaterialPassCustomization::VectorResetToDefault);
				FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);

				ChildHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCompositingMaterialPassCustomization::OnVectorOverrideChanged, ChildHandle));

				TSharedPtr<SWidget> NameWidget;
				TSharedPtr<SWidget> ValueWidget;
				FDetailWidgetRow Row;

				IDetailPropertyRow& PropertyRow = MaterialParametersGroup.AddPropertyRow(ChildHandle.ToSharedRef());

				PropertyRow.OverrideResetToDefault(ResetOverride);
				PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
				PropertyRow
					.ShowPropertyButtons(false)
					.IsEnabled(!IsRequiredParam)
					.CustomWidget()
					.NameContent()
					[

						SNew(SBox)
						.Padding(FMargin(15, 0))
						.Content()
						[
							SNew(STextBlock)
							.Text(FText::FromString(VectorName.ToString()))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
					.ValueContent()
					[
						ValueWidget.ToSharedRef()
					];
			}
		}
	}
}

void FCompositingMaterialPassCustomization::VectorResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (UMaterialInterface* MaterialReferencePtr = MaterialReference.Get())
	{
		FName VectorName;
		PropertyHandle->GetKeyHandle()->GetValue(VectorName);

		void* Data;
		PropertyHandle->GetValueData(Data);
		FLinearColor* VectorColor = reinterpret_cast<FLinearColor*>(Data);

		FLinearColor DefaultColor;
		MaterialReferencePtr->GetVectorParameterDefaultValue(VectorName, DefaultColor);
		if (FCompositingMaterialType* MatPass = GetMaterialPass())
		{
			MatPass->ResetVectorOverride(VectorName);
		}
		*VectorColor = DefaultColor;

	}
}

bool FCompositingMaterialPassCustomization::VectorShouldShowResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (UMaterialInterface* MaterialReferencePtr = MaterialReference.Get())
	{
		FName VectorName;
		PropertyHandle->GetKeyHandle()->GetValue(VectorName);

		void* Data;
		PropertyHandle->GetValueData(Data);
		FLinearColor* VectorColor = reinterpret_cast<FLinearColor*>(Data);

		FLinearColor DefaultColor;
		MaterialReferencePtr->GetVectorParameterDefaultValue(VectorName, DefaultColor);
		if (VectorColor && DefaultColor != *VectorColor)
		{
			return true;
		}

	}
	return false;
}

void FCompositingMaterialPassCustomization::OnVectorOverrideChanged(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (UMaterialInterface* MaterialReferencePtr = MaterialReference.Get())
	{
		FName VectorName;
		PropertyHandle->GetKeyHandle()->GetValue(VectorName);

		void* Data;
		PropertyHandle->GetValueData(Data);
		FLinearColor* VectorColor = reinterpret_cast<FLinearColor*>(Data);

		FLinearColor DefaultColor;
		MaterialReferencePtr->GetVectorParameterDefaultValue(VectorName, DefaultColor);
		if (FCompositingMaterialType* MatPass = GetMaterialPass())
		{
			if (DefaultColor == *VectorColor)
			{
				MatPass->ResetVectorOverride(VectorName);
			}
			else
			{
				MatPass->SetVectorOverride(VectorName, *VectorColor);
			}
			CompElementManager->RequestRedraw();
		}
	}
}

TOptional<float> FCompositingMaterialPassCustomization::GetScalarParameterSliderMin(FMaterialParameterInfo ScalarParam) const
{
	float Min = 0.0f, Max = 0.0f;
	if (UMaterialInterface* MaterialReferencePtr = MaterialReference.Get())
	{
		MaterialReferencePtr->GetScalarParameterSliderMinMax(ScalarParam, Min, Max);
	}
	return (Min == Max ? TOptional<float>() : Min);
}

TOptional<float> FCompositingMaterialPassCustomization::GetScalarParameterSliderMax(FMaterialParameterInfo ScalarParam) const
{
	float Min = 0.0f, Max = 0.0f;
	if (UMaterialInterface* MaterialReferencePtr = MaterialReference.Get())
	{
		MaterialReferencePtr->GetScalarParameterSliderMinMax(ScalarParam, Min, Max);
	}
	return (Min == Max ? TOptional<float>() : Max);
}

void FCompositingMaterialPassCustomization::OnScalarParameterSlideBegin(FMaterialParameterInfo ScalarParam)
{
	UE_LOGF(LogTemp, Warning, "Begin slide");

	GEditor->BeginTransaction(LOCTEXT("ChangeScalarParam", "Change Scalar Param"));

	if (TSharedPtr<IPropertyHandle> CachedPropertyHandlePin = CachedPropertyHandle.Pin())
	{
		TArray<UObject*> OuterObjects;
		CachedPropertyHandlePin->GetOuterObjects(OuterObjects);
		for (UObject* Obj : OuterObjects)
		{
			Obj->Modify();
		}
	}
}

void FCompositingMaterialPassCustomization::OnScalarParameterSlideEnd(const float NewValue, FMaterialParameterInfo ScalarParam)
{
	GEditor->EndTransaction();
}

void FCompositingMaterialPassCustomization::OnScalarParameterCommitted(const float NewValue, ETextCommit::Type CommitType, FMaterialParameterInfo ScalarParam)
{
	SetScalarParameterValue(NewValue, ScalarParam);
}

TOptional<float> FCompositingMaterialPassCustomization::GetScalarParameterValue(FMaterialParameterInfo ScalarParam) const
{
	float OutVal = 0.0f;

	FCompositingMaterialType* MatPass = GetMaterialPass();
	if (MatPass && MatPass->GetScalarOverride(ScalarParam.Name, OutVal))
	{
		return OutVal;
	}

	if (UMaterialInterface* MaterialReferencePtr = MaterialReference.Get())
	{
		MaterialReferencePtr->GetScalarParameterDefaultValue(ScalarParam, OutVal);
	}

	return OutVal;
}

void FCompositingMaterialPassCustomization::SetScalarParameterValue(const float NewValue, FMaterialParameterInfo ScalarParam)
{
	FCompositingMaterialType* MatPass = GetMaterialPass();
	if (!MatPass)
	{
		return;
	}

	float OutVal = 0.0f;
	MatPass->GetScalarOverride(ScalarParam.Name, OutVal);
	if (NewValue != OutVal)
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeScalarParam", "Change Scalar Param"));
		
		if (TSharedPtr<IPropertyHandle> CachedPropertyHandlePin = CachedPropertyHandle.Pin())
		{
			TArray<UObject*> OuterObjects;
			CachedPropertyHandlePin->GetOuterObjects(OuterObjects);
			for (UObject* Obj : OuterObjects)
			{
				Obj->Modify();
			}
		}

		if (UMaterialInterface* MaterialReferencePtr = MaterialReference.Get())
		{
			float ParamDefault = 0.0f;
			MaterialReferencePtr->GetScalarParameterDefaultValue(ScalarParam, ParamDefault);
			if (NewValue == ParamDefault)
			{
				MatPass->ResetScalarOverride(ScalarParam.Name);
			}
			else
			{
				MatPass->SetScalarOverride(ScalarParam.Name, NewValue);
			}
		}
		else
		{
			MatPass->SetScalarOverride(ScalarParam.Name, NewValue);
		}

		CompElementManager->RequestRedraw();
	}
}

EVisibility FCompositingMaterialPassCustomization::IsResetScalarParameterVisible(FMaterialParameterInfo ScalarParam) const
{
	float OverrideVal = 0.0f;
	FCompositingMaterialType* MatPass = GetMaterialPass();
	return (MatPass && MatPass->GetScalarOverride(ScalarParam.Name, OverrideVal))
		? EVisibility::Visible
		: EVisibility::Hidden;
}

FReply FCompositingMaterialPassCustomization::OnResetScalarParameterClicked(FMaterialParameterInfo ScalarParam)
{
	FScopedTransaction Transaction(LOCTEXT("ResetScalarParam", "Reset Scalar Param"));

	if (TSharedPtr<IPropertyHandle> CachedPropertyHandlePin = CachedPropertyHandle.Pin())
	{
		TArray<UObject*> OuterObjects;
		CachedPropertyHandlePin->GetOuterObjects(OuterObjects);
		for (UObject* Obj : OuterObjects)
		{
			Obj->Modify();
		}
	}

	if (FCompositingMaterialType* MatPass = GetMaterialPass())
	{
		MatPass->ResetScalarOverride(ScalarParam.Name);
		CompElementManager->RequestRedraw();
	}

	return FReply::Handled();
}

void FCompositingMaterialPassCustomization::HandleRequiredParamComboChanged(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo, TWeakPtr<SComboButton> ComboButtonHandle, FName ParamName)
{
	FCompositingMaterialType* MatPass = GetMaterialPass();
	if (Item.IsValid() && MatPass)
	{
		FScopedTransaction Transaction(LOCTEXT("RequiredParamUpdated", "Update Required Parameter"));

		if (TSharedPtr<IPropertyHandle> CachedPropertyHandlePin = CachedPropertyHandle.Pin())
		{
			TArray<UObject*> OuterObjects;
			CachedPropertyHandlePin->GetOuterObjects(OuterObjects);
			for (UObject* Obj : OuterObjects)
			{
				Obj->Modify();
			}
		}

		MatPass->RequiredMaterialParams[ParamName].ParamName = *Item;
		GEditor->RedrawAllViewports(/*bInvalidateHitProxies =*/false);
		
		if (CachedUtils)
		{
			CachedUtils->ForceRefresh();
		}
	}

	if (ComboButtonHandle.IsValid())
	{
		ComboButtonHandle.Pin()->SetIsOpen(false);
	}
}

FText FCompositingMaterialPassCustomization::GetRequiredParamComboText(FName ParamName) const
{
	FName ParamNameOut = NAME_None;
	if (FCompositingMaterialType* MatPass = GetMaterialPass())
	{
		ParamNameOut = MatPass->RequiredMaterialParams[ParamName];
	}
	/*
	//TODO: Should this apply to required params too?
	if (ParamName.IsNone())
	{
		static const FName PrePassSelfAlias = TEXT("Self");
		static const FName PrePassParamName = TEXT("PrePass");
		IConsoleVariable* CVarUserPrePassParamName = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Composure.CompositingElements.InternalPrePassParamName"));
		FName PrePassUserAlias = CVarUserPrePassParamName ? FName(*CVarUserPrePassParamName->GetString()) : PrePassParamName;

		if (TexName == PrePassUserAlias || TexName == PrePassSelfAlias || TexName == PrePassParamName)
		{
			ParamName = PrePassParamName;
		}
		else
		{
			for (TSharedPtr<FName> NamePtr : TextureComboSource)
			{
				if (TexName == *NamePtr.Get())
				{
					ParamName = TexName;
					break;
				}
			}
		}
	}
	*/
	return FText::FromName(ParamNameOut);
}

void FCompositingMaterialPassCustomization::OnRequiredParamComboTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName ParamName)
{
	if (FCompositingMaterialType* MatPass = GetMaterialPass())
	{
		FScopedTransaction Transaction(LOCTEXT("RequiredParamUpdated", "Update Required Parameter"));

		if (TSharedPtr<IPropertyHandle> CachedPropertyHandlePin = CachedPropertyHandle.Pin())
		{
			TArray<UObject*> OuterObjects;
			CachedPropertyHandlePin->GetOuterObjects(OuterObjects);
			for (UObject* Obj : OuterObjects)
			{
				Obj->Modify();
			}
		}

		MatPass->RequiredMaterialParams[ParamName].ParamName = FName(*NewText.ToString());
		
		if (CachedUtils)
		{
			CachedUtils->ForceRefresh();
		}
		GEditor->RedrawAllViewports(/*bInvalidateHitProxies =*/false);
	}
}

TSharedRef<SWidget> FCompositingMaterialPassCustomization::GetRequiredParamComboMenu(TWeakPtr<SComboButton> ComboButtonHandle, FName ParamName, EParamType ParamType)
{
	//ReBuild combobox sources
	RebuildRequiredParamSources();

	TArray<TSharedPtr<FName>>* ComboSource = nullptr;

	switch (ParamType)
	{
	case EParamType::ScalarParam:
		ComboSource = &RequiredParamComboSourceScalar;
		break;
	case EParamType::VectorParam:
		ComboSource = &RequiredParamComboSourceVector;
		break;
	case EParamType::TextureParam:
		ComboSource = &RequiredParamComboSourceTexture;
		break;
	case EParamType::MediaTextureParam:
		ComboSource = &RequiredParamComboSourceMedia;
		break;
	case EParamType::UnknownParamType:
		ComboSource = &RequiredParamComboSourceUnknown;
		break;
		//default:
			//Source list not found
	}



	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(400.0f)
		[
			SNew(SListView< TSharedPtr<FName> >)
			.ListItemsSource(ComboSource)
		.OnGenerateRow(this, &FCompositingMaterialPassCustomization::GenerateComboItem)
		.OnSelectionChanged(this, &FCompositingMaterialPassCustomization::HandleRequiredParamComboChanged, ComboButtonHandle, ParamName)
		];
}

TSharedRef<ITableRow> FCompositingMaterialPassCustomization::GenerateComboItem(TSharedPtr<FName> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FName>>, OwnerTable)
		[
			SNew(STextBlock).Text(FText::FromName(*InItem))
		];
}

void FCompositingMaterialPassCustomization::HandleComboChanged(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo, TWeakPtr<SComboButton> ComboButtonHandle, FName TexName)
{
	FCompositingMaterialType* MatPass = GetMaterialPass();
	if (Item.IsValid() && MatPass)
	{
		FScopedTransaction Transaction(LOCTEXT("InputElementUpdated", "Update Input Element"));

		if (TSharedPtr<IPropertyHandle> CachedPropertyHandlePin = CachedPropertyHandle.Pin())
		{
			TArray<UObject*> OuterObjects;
			CachedPropertyHandlePin->GetOuterObjects(OuterObjects);
			for (UObject* Obj : OuterObjects)
			{
				Obj->Modify();
			}
		}

		MatPass->ParamPassMappings[TexName] = *Item;
		GEditor->RedrawAllViewports(/*bInvalidateHitProxies =*/false);
	}

	if (ComboButtonHandle.IsValid())
	{
		ComboButtonHandle.Pin()->SetIsOpen(false);
	}
}

FText FCompositingMaterialPassCustomization::GetComboText(FName TexName) const
{
	FName ParamName = NAME_None;
	if (FCompositingMaterialType* MatPass = GetMaterialPass())
	{
		if (MatPass->ParamPassMappings.Contains(TexName))
		{
			ParamName = MatPass->ParamPassMappings[TexName];
		}
	}

	if (ParamName.IsNone())
	{
		static const FName PrePassSelfAlias = TEXT("Self");
		static const FName PrePassParamName = TEXT("PrePass");
		IConsoleVariable* CVarUserPrePassParamName = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Composure.CompositingElements.InternalPrePassParamName"));
		FName PrePassUserAlias = CVarUserPrePassParamName ? FName(*CVarUserPrePassParamName->GetString()) : PrePassParamName;

		if (TexName == PrePassUserAlias || TexName == PrePassSelfAlias || TexName == PrePassParamName)
		{
			ParamName = PrePassParamName;
		}
		else
		{
			for (TSharedPtr<FName> NamePtr : TextureComboSource)
			{
				if (TexName == *NamePtr.Get())
				{
					ParamName = TexName;
					break;
				}
			}
		}
	}

	return FText::FromName(ParamName);
}

void FCompositingMaterialPassCustomization::OnComboTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName TexName)
{
	if (FCompositingMaterialType* MatPass = GetMaterialPass())
	{
		FScopedTransaction Transaction(LOCTEXT("InputElementUpdated", "Update Input Element"));

		if (TSharedPtr<IPropertyHandle> CachedPropertyHandlePin = CachedPropertyHandle.Pin())
		{
			TArray<UObject*> OuterObjects;
			CachedPropertyHandlePin->GetOuterObjects(OuterObjects);
			for (UObject* Obj : OuterObjects)
			{
				Obj->Modify();
			}
		}

		MatPass->ParamPassMappings.Add(TexName, *NewText.ToString());
		GEditor->RedrawAllViewports(/*bInvalidateHitProxies =*/false);
	}
}

TSharedRef<SWidget> FCompositingMaterialPassCustomization::GetPassComboMenu(TWeakPtr<SComboButton> ComboButtonHandle, FName TexName)
{
	//ReBuild combobox source
	RebuildTextureSourceList();

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(400.0f)
		[
			SNew(SListView< TSharedPtr<FName> >)
			.ListItemsSource(&TextureComboSource)
			.OnGenerateRow(this, &FCompositingMaterialPassCustomization::GenerateComboItem)
			.OnSelectionChanged(this, &FCompositingMaterialPassCustomization::HandleComboChanged, ComboButtonHandle, TexName)
		];
}

void FCompositingMaterialPassCustomization::TextureResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, TSharedPtr<SComboButton> ComboBoxHandle)
{
	PropertyHandle->SetValue(FName(NAME_None));
	//ComboBoxHandle->ClearSelection();
}

bool FCompositingMaterialPassCustomization::TextureShouldShowResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FName ValueName;
	PropertyHandle->GetValue(ValueName);
	return (!ValueName.IsNone());
}

void FCompositingMaterialPassCustomization::RebuildTextureSourceList()
{
	TWeakObjectPtr<ACompositingElement> CompElement;
	TextureComboSource.Reset();

	TArray<UObject*> OuterChain;
	if (TSharedPtr<IPropertyHandle> CachedPropertyHandlePin = CachedPropertyHandle.Pin())
	{
		TArray<UObject*> OuterObjects;
		CachedPropertyHandlePin->GetOuterObjects(OuterObjects);

		if (OuterObjects.Num() == 1)
		{
			UObject* Outer = OuterObjects[0];
			while (Outer != nullptr && !Outer->IsA<ACompositingElement>())
			{
				OuterChain.Add(Outer);
				Outer = Outer->GetOuter();
			}
			if (ACompositingElement* AsElement = Cast<ACompositingElement>(Outer))
			{
				CompElement = AsElement;
			}
		}
	}

	if (CompElement.IsValid())
	{
		ACompositingElement* CompElementPtr = CompElement.Get();

		// First get earlier passes on this element

		TArray<FName> InternalPassOptions;
		InternalPassOptions.Reserve(TextureComboSource.GetSlack());

		auto RecordAvailablePass = [&InternalPassOptions](const FName PassName)
		{
			if (!PassName.IsNone())
			{
				// Since passes can technically override the same name, we need to make sure that this list contains unique entries.
				// We do a Remove() instead of an AddUnique() because we want the list order to reflect render order, and its the
				// later pass that would overwrite the earlier one.
				InternalPassOptions.Remove(PassName);
				InternalPassOptions.Add(PassName);
			}
		};

		bool FoundSelf = false;
		auto IsSelf = [OuterChain](UCompositingElementPass* Pass)->bool
		{
			return OuterChain.Contains(Pass);
		};

		TArray<FName> InputIntermediates;
		auto ClearAllInputIntermediates = [&InputIntermediates, &InternalPassOptions]()
		{
			for (const FName& Intermediate : InputIntermediates)
			{
				InternalPassOptions.Remove(Intermediate);
			}
		};

		for (UCompositingElementInput* Input : CompElementPtr->GetInputsList())
		{
			if (IsSelf(Input))
			{
				FoundSelf = true;
				break;
			}
			else if (Input && Input->IsPassEnabled())
			{
				if (Input->bIntermediate)
				{
					InputIntermediates.Add(Input->PassName);
				}
				else
				{
					// Since passes can technically override names, we want to make sure that an earlier intermediate
					// doesn't keep this pass from being in the list
					InputIntermediates.Remove(Input->PassName);
				}
				RecordAvailablePass(Input->PassName);
			}
		}

		FName IntermediatePassName = NAME_None;
		if (!FoundSelf)
		{
			for (UCompositingElementTransform* Transform : CompElementPtr->GetTransformsList())
			{
				if (IsSelf(Transform))
				{
					if (!IntermediatePassName.IsNone())
					{
						// Transforms that are intermediate are available to the pass that immediately follows.
						RecordAvailablePass(IntermediatePassName);
					}

					FoundSelf = true;
					break;
				}
				else if (Transform->IsPassEnabled())
				{
					if (Transform->bIntermediate)
					{
						// Intermediate transforms are only available to the following pass,
						// so don't add this to the list yet (add it once we've found 'Self')
						IntermediatePassName = Transform->PassName;
					}
					else
					{
						IntermediatePassName = NAME_None;
						RecordAvailablePass(Transform->PassName);
					}
					// If 'Self' was the first transform, then we'd have all the inputs available to us,
					// otherwise the intermediate ones get returned to the pool and cannot be used
					ClearAllInputIntermediates();
				}
			}
		}

		if (!FoundSelf)
		{
			TArray<UCompositingElementOutput*> Outputs = CompElementPtr->GetOutputsList();
			for (UObject* Outer : OuterChain)
			{
				if (Outputs.Contains(Cast<UCompositingElementOutput>(Outer)))
				{
					if (!IntermediatePassName.IsNone())
					{
						RecordAvailablePass(IntermediatePassName);
					}

					FoundSelf = true;
					break;
				}
			}

			if (!FoundSelf)
			{
				// Since we didn't find this material in any of the set passes, we can't guarantee where this material 
				// used in the element's pipeline, so don't offer any internal passes as options.
				InternalPassOptions.Empty();
			}
		}

		if (InternalPassOptions.Num() > 0)
		{
			TextureComboSource.Add(MakeShared<FName>(TEXT("PrePass")));
		}
		for (FName InternalPassName : InternalPassOptions)
		{
			TextureComboSource.Add(MakeShared<FName>(InternalPassName));
		}

		//Now, get all children passes recursively
		const TArray<ACompositingElement*> Children = CompElementPtr->GetChildElements();
		for (ACompositingElement* Element : Children)
		{
			if (Element)
			{
				TextureComboSource.Append(GetPassNamesRecursive(Element));
			}
		}
	}

	//TextureComboSource.Sort([](const TSharedPtr<FName> A, const TSharedPtr<FName> B) { return (A->Compare(*B) < 0); });
}

void FCompositingMaterialPassCustomization::RebuildRequiredParamSources()
{
	UMaterialInterface* MaterialReferencePtr = MaterialReference.Get();
	check(MaterialReferencePtr);

	RequiredParamComboSourceUnknown.Reset();

	RequiredParamComboSourceScalar.Reset();
	TArray<FMaterialParameterInfo> OutScalarParameterInfo;
	TArray<FGuid> ScalarGuids;
	MaterialReferencePtr->GetAllScalarParameterInfo(OutScalarParameterInfo, ScalarGuids);
	for (FMaterialParameterInfo Param : OutScalarParameterInfo)
	{
		RequiredParamComboSourceScalar.Add(MakeShared<FName>(Param.Name));
		RequiredParamComboSourceUnknown.Add(MakeShared<FName>(Param.Name));
	}

	RequiredParamComboSourceVector.Reset();
	TArray<FMaterialParameterInfo> OutVectorParameterInfo;
	TArray<FGuid> VectorGuids;
	MaterialReferencePtr->GetAllVectorParameterInfo(OutVectorParameterInfo, VectorGuids);
	for (FMaterialParameterInfo Param : OutVectorParameterInfo)
	{
		RequiredParamComboSourceVector.Add(MakeShared<FName>(Param.Name));
		RequiredParamComboSourceUnknown.Add(MakeShared<FName>(Param.Name));
	}

	RequiredParamComboSourceTexture.Reset();
	TArray<FMaterialParameterInfo> OutTextureParameterInfo;
	TArray<FGuid> TextureGuids;
	MaterialReferencePtr->GetAllTextureParameterInfo(OutTextureParameterInfo, TextureGuids);
	for (FMaterialParameterInfo Param : OutTextureParameterInfo)
	{
		RequiredParamComboSourceTexture.Add(MakeShared<FName>(Param.Name));
		RequiredParamComboSourceUnknown.Add(MakeShared<FName>(Param.Name));
	}


	//TODO: Media Texture Params?


}

TArray<TSharedPtr<FName>> FCompositingMaterialPassCustomization::GetPassNamesRecursive(ACompositingElement* Element, const FString& InPrefix)
{
	TArray<TSharedPtr<FName>> NamesToAdd;

	FString Prefix = InPrefix;
	auto AddPassNameToList = [&NamesToAdd, &Prefix](FName PassName)
	{
		if (!PassName.IsNone())
		{
			FString PathName = Prefix + PassName.ToString();
			NamesToAdd.Add(MakeShared<FName>(*PathName));
		}
	};

	AddPassNameToList(Element->GetCompElementName());
	Prefix += Element->GetCompElementName().ToString() + TEXT('.');

	for (UCompositingElementInput* Input : Element->GetInputsList())
	{
		if (Input->IsPassEnabled() && !Input->bIntermediate)
		{
			AddPassNameToList(Input->PassName);
		}
	}

	for (UCompositingElementTransform* Transform : Element->GetTransformsList())
	{
		if (Transform->IsPassEnabled() && !Transform->bIntermediate)
		{
			AddPassNameToList(Transform->PassName);
		}
	}

	// NOTE: Outputs aren't available, as they do not return a texture/target to source from.


	for (ACompositingElement* ChildElement : Element->GetChildElements())
	{
		if (ChildElement)
		{
			NamesToAdd.Append(GetPassNamesRecursive(ChildElement, Prefix));
		}
	}

	return NamesToAdd;
}

FCompositingMaterialType* FCompositingMaterialPassCustomization::GetMaterialPass() const
{
	if (TSharedPtr<IPropertyHandle> CachedPropertyHandlePin = CachedPropertyHandle.Pin())
	{
		TArray<void*> RawData;
		CachedPropertyHandlePin->AccessRawData(RawData);

		if (RawData.Num() > 0)
		{
			return reinterpret_cast<FCompositingMaterialType*>(RawData[0]);
		}		
	}
	return nullptr;
}

void FCompositingMaterialPassCustomization::ResetParameterOverrides()
{
	if (FCompositingMaterialType* MatPass = GetMaterialPass())
	{
		MatPass->ResetAllParamOverrides();
		MatPass->UpdateProxyMap();

		if (CachedUtils)
		{
			CachedUtils->ForceRefresh();
		}
	}
}

void FCompositingMaterialPassCustomization::OnRedrawViewports()
{
	if (FCompositingMaterialType* MatPass = GetMaterialPass())
	{
		MatPass->UpdateProxyMap();

		if (CachedUtils)
		{
			CachedUtils->ForceRefresh();
		}
	}
}

void FCompositingMaterialPassCustomization::PostUndo(bool bSuccess)
{
	if (FCompositingMaterialType* MatPass = GetMaterialPass())
	{
		MatPass->MarkDirty();
		MatPass->ApplyParamOverrides(nullptr);
		CompElementManager->RequestRedraw();
	}
}

/* FCompositingPassCustomization
 *****************************************************************************/

#include "Widgets/SCompElementPreviewDialog.h"

TSharedRef<IPropertyTypeCustomization> FCompositingPassCustomization::MakeInstance()
{
	return MakeShareable(new FCompositingPassCustomization);
}

void FCompositingPassCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedRef<SWidget> DefaultValWidget = PropertyHandle->CreatePropertyValueWidget(/*bDisplayDefaultPropertyButtons =*/false);
	HeaderValueWidget = TSharedPtr<SWidget>(DefaultValWidget);

	HeaderRow
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
			// Match the same Min/Max from ConstructPropertyEditorWidget()
			.MinDesiredWidth(250.f)
			.MaxDesiredWidth(600.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
		[
			DefaultValWidget		
		];

	if (PropertyHandle->GetIndexInArray() != INDEX_NONE)
	{
		TSharedPtr<IPropertyHandle> PassNameHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UCompositingElementPass, PassName));
		if (PassNameHandle.IsValid())
		{
			FName NameValue;
			PassNameHandle->GetValue(NameValue);

			if (!NameValue.IsNone())
			{
				FText PassNameText;
				PassNameHandle->GetValueAsDisplayText(PassNameText);

				HeaderRow.NameContent()
				[
					PropertyHandle->CreatePropertyNameWidget(PassNameText)
				];
			}			
		}
	}
}

void FCompositingPassCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	const bool bShowInnerPropsOnly = PropertyHandle->HasMetaData(TEXT("ShowOnlyInnerProperties"));

	TSharedPtr<IPropertyHandle> ObjectHandle = GetInstancedObjectHandle(PropertyHandle);
	if (ObjectHandle.IsValid())
	{
		TSharedPtr<SWidget> ParentWidget  = HeaderValueWidget.IsValid() ? HeaderValueWidget.Pin() : nullptr;
		TSharedPtr<SWidget> PreviewButton = ConditionallyCreatePreviewButton(PropertyHandle, ParentWidget);
		if (PreviewButton.IsValid())
		{
			ChildBuilder.AddCustomRow(LOCTEXT("PreviewLabel", "Preview"))
				.ValueContent()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
				[
					SNew(SBox)
						.HAlign(HAlign_Right)
					[
						PreviewButton.ToSharedRef()
					]
				];
		}

		PropertyHandle = ObjectHandle.ToSharedRef();		
	}

	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();

		if (bShowInnerPropsOnly)
		{
			uint32 NumGrandChildren;
			ChildHandle->GetNumChildren(NumGrandChildren);

			for (uint32 GrandChildIndex = 0; GrandChildIndex < NumGrandChildren; ++GrandChildIndex)
			{
				TSharedRef<IPropertyHandle> GrandChildHandle = ChildHandle->GetChildHandle(GrandChildIndex).ToSharedRef();
				ChildBuilder.AddProperty(GrandChildHandle);
			}
		}
		else
		{
			ChildBuilder.AddProperty(ChildHandle);
		}			
	}
}

TSharedPtr<IPropertyHandle> FCompositingPassCustomization::GetInstancedObjectHandle(TSharedRef<IPropertyHandle> PropertyHandle)
{
	TSharedPtr<IPropertyHandle> ChildHandle;

	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);

	if (NumChildren > 0)
	{
		const bool bIsEditInlineObj = PropertyHandle->HasMetaData(TEXT("EditInline"));
		if (bIsEditInlineObj)
		{
			// when the property is a (inlined) object property, the first child will be
			// the object instance, and its properties are the children underneath that
			ensure(NumChildren == 1);

			ChildHandle = PropertyHandle->GetChildHandle(0);
		}
	}

	return ChildHandle;
}

TSharedPtr<SWidget> FCompositingPassCustomization::ConditionallyCreatePreviewButton(TSharedRef<IPropertyHandle> PropertyHandle, TSharedPtr<SWidget> ParentWidget)
{
	TSharedPtr<SWidget> PreviewButton;

	TSharedPtr<IPropertyHandle> ObjectHandle = GetInstancedObjectHandle(PropertyHandle);
	if (ObjectHandle.IsValid())
	{
		PropertyHandle = ObjectHandle.ToSharedRef();

		TArray<void*> RawData;
		ObjectHandle->AccessRawData(RawData);

		if (RawData.Num() > 0)
		{
			UCompositingElementPass* PassObj = reinterpret_cast<UCompositingElementPass*>(RawData[0]);

			if (PassObj && PassObj->Implements<UCompEditorImagePreviewInterface>())
			{
				TWeakUIntrfacePtr<ICompEditorImagePreviewInterface> WeakPassPtr(PassObj);

				PreviewButton = SNew(SButton)
					[
						SNew(STextBlock).Text(LOCTEXT("PreviewLabel", "Preview"))
					]						
					.OnClicked_Lambda([WeakPassPtr, ParentWidget, PassObj]()->FReply
					{
						FText WindowTitle;
						if (WeakPassPtr.IsValid())
						{
							WindowTitle = FText::Format(LOCTEXT("PreviewWindowTitle", "Preview: {0}"), FText::FromName(PassObj->PassName));
						}
					
						SCompElementPreviewDialog::OpenPreviewWindow(WeakPassPtr, ParentWidget, WindowTitle);
						return FReply::Handled();
					});
			}
		}
	}
	return PreviewButton;
}

#undef LOCTEXT_NAMESPACE
