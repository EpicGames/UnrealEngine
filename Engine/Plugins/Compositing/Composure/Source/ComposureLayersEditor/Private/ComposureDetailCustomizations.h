// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IDetailCustomization.h"
#include "EditorUndoClient.h"
#include "IPropertyTypeCustomization.h"

class ACompositingElement;
class FReply;
class ICompElementManager;
class SComboButton;
class SToolTip;
class SWidget;
class UMaterialInterface;
enum class EParamType : uint8;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
struct EVisibility;
struct FCompositingMaterial;
struct FMaterialParameterInfo;
template <typename OptionalType> struct TOptional;

/* FCompElementDetailsCustomization
 *****************************************************************************/

class IDetailLayoutBuilder;

class FCompElementDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

public:
	//~ IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	void ForceRefreshLayout();
	void GetInstanceCameraSourceComboStrings(TArray<TSharedPtr<FString>>& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems);
	FString GetInstanceCameraSourceValueStr(TSharedPtr<IPropertyHandle> PropertyHandle);
	void OnCameraSourceSelected(const FString& Selection, TSharedPtr<IPropertyHandle> PropertyHandle);

	IDetailLayoutBuilder* MyLayout = nullptr;
};

/* FCompositingMaterialPassCustomization
 *****************************************************************************/

class ITableRow;
class STableViewBase;
typedef FCompositingMaterial FCompositingMaterialType;

class FCompositingMaterialPassCustomization : public IPropertyTypeCustomization, public FEditorUndoClient
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FCompositingMaterialPassCustomization();
	~FCompositingMaterialPassCustomization();

	/** Begin IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** End IPropertyTypeCustomization interface */

	// Begin FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

protected:
	TSharedPtr<ICompElementManager> CompElementManager;
	TWeakPtr<IPropertyHandle> CachedPropertyHandle;
	TSharedPtr<IPropertyHandle> CachedVectorProxies;
	TSharedPtr<IPropertyHandle> CachedMaterialParamMappings;
	TSharedPtr<IPropertyUtilities> CachedUtils;
	TSharedPtr<IPropertyHandle> MaterialHandle;
	TWeakObjectPtr<UMaterialInterface> MaterialReference;
	FName* MaterialPassName;

private:
	/** Overrides for vector param reset to default*/
	void VectorResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);
	bool VectorShouldShowResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);
	void OnVectorOverrideChanged(TSharedPtr<IPropertyHandle> PropertyHandle);
	void TextureResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, TSharedPtr<SComboButton> ComboBoxHandle);
	bool TextureShouldShowResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);

	TOptional<float> GetScalarParameterSliderMin(FMaterialParameterInfo ScalarParam) const;
	TOptional<float> GetScalarParameterSliderMax(FMaterialParameterInfo ScalarParam) const;
	void OnScalarParameterSlideBegin(FMaterialParameterInfo ScalarParam);
	void OnScalarParameterSlideEnd(const float NewValue, FMaterialParameterInfo ScalarParam);
	void OnScalarParameterCommitted(const float NewValue, ETextCommit::Type CommitType, FMaterialParameterInfo ScalarParam);
	TOptional<float> GetScalarParameterValue(FMaterialParameterInfo ScalarParam) const;
	void SetScalarParameterValue(const float NewValue, FMaterialParameterInfo ScalarParam);
	EVisibility IsResetScalarParameterVisible(FMaterialParameterInfo ScalarParam) const;
	FReply OnResetScalarParameterClicked(FMaterialParameterInfo ScalarParam);

	void HandleRequiredParamComboChanged(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo, TWeakPtr<SComboButton> ComboButtonHandle, FName ParamName);
	FText GetRequiredParamComboText(FName TexName) const;
	void OnRequiredParamComboTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName ParamName);
	TSharedRef<SWidget> GetRequiredParamComboMenu(TWeakPtr<SComboButton> ComboButtonHandle, FName ParamName, EParamType ParamType);
	void RebuildRequiredParamSources();

	TSharedRef<ITableRow> GenerateComboItem(TSharedPtr<FName> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void HandleComboChanged(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo, TWeakPtr<SComboButton> ComboButtonHandle, FName TexName);
	FText GetComboText(FName TexName) const;
	void OnComboTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName TexName);
	TSharedRef<SWidget> GetPassComboMenu(TWeakPtr<SComboButton> ComboButtonHandle, FName TexName);
	void RebuildTextureSourceList();

	static TArray<TSharedPtr<FName>> GetPassNamesRecursive(ACompositingElement* Element, const FString& Prefix = FString());
	void ResetParameterOverrides();
	void OnRedrawViewports();
	FCompositingMaterialType* GetMaterialPass() const;

	TArray<TSharedPtr<FName>> TextureComboSource;

	TArray<TSharedPtr<FName>> RequiredParamComboSourceScalar;
	TArray<TSharedPtr<FName>> RequiredParamComboSourceVector;
	TArray<TSharedPtr<FName>> RequiredParamComboSourceTexture;
	TArray<TSharedPtr<FName>> RequiredParamComboSourceMedia;
	TArray<TSharedPtr<FName>> RequiredParamComboSourceUnknown;
};


/* FCompositingPassCustomization
 *****************************************************************************/

class FCompositingPassCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:
	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization interface

private: 
	TSharedPtr<IPropertyHandle> GetInstancedObjectHandle(TSharedRef<IPropertyHandle> PropertyHandle);
	TSharedPtr<SWidget> ConditionallyCreatePreviewButton(TSharedRef<IPropertyHandle> PropertyHandle, TSharedPtr<SWidget> ParentWidget);

	TWeakPtr<SWidget> HeaderValueWidget;
};
