// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComposureEditorSettings.h"

UDefaultComposureEditorSettings::UDefaultComposureEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

UComposureEditorSettings::UComposureEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

const TArray<FSoftObjectPath>& UComposureEditorSettings::GetFeaturedCompShotClasses() const
{
	if (FeaturedCompShotClassOverrides.Num() > 0)
	{
		return FeaturedCompShotClassOverrides;
	}

	const UDefaultComposureEditorSettings* DefaultSettings = GetDefault<UDefaultComposureEditorSettings>();
	return DefaultSettings->FeaturedCompShotClasses;
}

const TArray<FSoftObjectPath>& UComposureEditorSettings::GetFeaturedElementClasses() const
{
	if (FeaturedElementClassOverrides.Num() > 0)
	{
		return FeaturedElementClassOverrides;
	}

	const UDefaultComposureEditorSettings* DefaultSettings = GetDefault<UDefaultComposureEditorSettings>();
	return DefaultSettings->FeaturedElementClasses;
}