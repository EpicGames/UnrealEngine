// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComposureEditorSettings.generated.h"

UCLASS(config=Composure)
class UDefaultComposureEditorSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(Config)
	TArray<FSoftObjectPath> FeaturedCompShotClasses;

	UPROPERTY(Config)
	TArray<FSoftObjectPath> FeaturedElementClasses;

	UPROPERTY(Config)
	TMap<FName, FString> DefaultElementNames;
};

UCLASS(config=Editor)
class UComposureEditorSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(Config)
	TArray<FSoftObjectPath> FeaturedCompShotClassOverrides;

	UPROPERTY(Config)
	TArray<FSoftObjectPath> FeaturedElementClassOverrides;

	// falls back to the defaults, if the project didn't specify any
	const TArray<FSoftObjectPath>& GetFeaturedCompShotClasses() const;
	const TArray<FSoftObjectPath>& GetFeaturedElementClasses() const;
};
