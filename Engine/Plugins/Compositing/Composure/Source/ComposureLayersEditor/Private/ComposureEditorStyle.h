// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateColorBrush.h"
#include "Styling/SlateStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/SlateTypes.h"
#include "Misc/Paths.h"

#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BOX_BRUSH(RelativePath, ...) FSlateBoxBrush(RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BORDER_BRUSH(RelativePath, ...) FSlateBorderBrush(RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define TTF_FONT(RelativePath, ...) FSlateFontInfo(RootToContentDir(RelativePath, TEXT(".ttf")), __VA_ARGS__)
#define OTF_FONT(RelativePath, ...) FSlateFontInfo(RootToContentDir(RelativePath, TEXT(".otf")), __VA_ARGS__)

class FComposureEditorStyle final
	: public FSlateStyleSet
{
public:
	FComposureEditorStyle()
		: FSlateStyleSet("ComposureEditorStyle")
	{
		const FVector2D Icon10x10(10.0f, 10.0f);
		const FVector2D Icon12x12(12.0f, 12.0f);
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon24x24(24.0f, 24.0f);
		const FVector2D Icon32x32(32.0f, 32.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Compositing/Composure/Content"));

		const FSlateColor DefaultForeground(FLinearColor(0.72f, 0.72f, 0.72f, 1.f)); 

		// Class Icons
		{
			Set("ClassIcon.CompositingElement", new IMAGE_BRUSH("Editor/Slate/Icons/icon_CompShot_16px", Icon16x16));
			Set("ClassIcon.BP_CgCaptureCompElement_C", new IMAGE_BRUSH("Editor/Slate/Icons/icon_CompElement_16x", Icon16x16));
			Set("ClassIcon.BP_MediaPlateCompElement_C", new IMAGE_BRUSH("Editor/Slate/Icons/icon_MediaPlate_16x", Icon16x16));
			Set("ClassIcon.BP_CgMatteCompElement_C", new IMAGE_BRUSH("Editor/Slate/Icons/icon_MaskLayer_16x", Icon16x16));
			Set("ClassIcon.BP_CgCaptureCompElement_C", new IMAGE_BRUSH("Editor/Slate/Icons/icon_CgCaptureLayer_16x", Icon16x16));
		}

		// Button Icons
		{
			Set("ComposureProperties.Button_ChromaPicker", new IMAGE_BRUSH("Editor/Slate/Icons/icon_ChromaPicker_12x", Icon12x12));

			Set("ComposureTree.FrameFrozenIcon16x", new IMAGE_BRUSH("Editor/Slate/Icons/icon_FrameFrozen_16px", Icon16x16));

			Set("ComposureTree.FrameFrozenHighlightIcon16x", new IMAGE_BRUSH("Editor/Slate/Icons/icon_FrameFrozen_hi_16px", Icon16x16));
			Set("ComposureTree.NoFreezeFrameIcon16x", new IMAGE_BRUSH("Editor/Slate/Icons/icon_FrameNotFreezed_16px", Icon16x16));
			Set("ComposureTree.NoFreezeFrameHighlightIcon16x", new IMAGE_BRUSH("Editor/Slate/Icons/icon_FrameNotFreezed_hi_16px", Icon16x16));

			Set("ComposureTree.MediaCaptureOn16x", new IMAGE_BRUSH("Editor/Slate/Icons/icon_MediaCapture_Active_16x", Icon16x16));
			Set("ComposureTree.MediaCaptureOff16x", new IMAGE_BRUSH("Editor/Slate/Icons/icon_MediaCapture_Inactive_16x", Icon16x16));
			Set("ComposureTree.NoMediaCapture16x", new IMAGE_BRUSH("Editor/Slate/Icons/icon_EmptyCheckbox_16x", Icon16x16));

			Set("CompPreviewPane.MaximizeWindow16x", new IMAGE_BRUSH("Editor/Slate/Icons/icon_MaximizeWindow_16px", Icon16x16));
		}

		// Button Styles
		{
			FButtonStyle ColorPickerPreviewButton = FButtonStyle()
				.SetNormal(FSlateNoResource())
				.SetHovered(FSlateNoResource())
				.SetPressed(FSlateNoResource())
				.SetNormalPadding(FMargin(0, 0, 0, 0))
				.SetPressedPadding(FMargin(0, 0, 0, 0));
			Set("ColorPickerPreviewButton", ColorPickerPreviewButton);
		}

		{
			Set("ComposureTree.AlphaHandle", new IMAGE_BRUSH("Editor/Slate/Images/AlphaHandle", FVector2D(11.f, 18.f)));
			const FSlateBrush* SliderHandle = GetBrush("ComposureTree.AlphaHandle");

			Set("ComposureTree.AlphaScrubber", FSliderStyle()
				.SetNormalBarImage(FSlateColorBrush(FColor::White))
				.SetHoveredBarImage(FSlateColorBrush(FColor::White))
				.SetDisabledBarImage(FSlateColorBrush(FLinearColor::Gray))
				.SetNormalThumbImage(*SliderHandle)
				.SetHoveredThumbImage(*SliderHandle)
				.SetDisabledThumbImage(*SliderHandle)
				.SetBarThickness(2.0f)
			);
		}

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FComposureEditorStyle& Get()
	{
		static FComposureEditorStyle Inst;
		return Inst;
	}
	
	~FComposureEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH
#undef TTF_FONT
#undef OTF_FONT
