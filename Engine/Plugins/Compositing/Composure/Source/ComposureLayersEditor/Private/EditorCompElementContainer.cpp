// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorCompElementContainer.h"
#include "Engine/World.h"
#include "UObject/UObjectIterator.h"
#include "CompositingElement.h"
#include "LevelUtils.h"

UEditorCompElementContainer::UEditorCompElementContainer()
{
	RebuildEditorElementsList();
}

bool UEditorCompElementContainer::Add(ACompositingElement* NewElement, const bool bTransactional)
{
	bool bAdded = false;
	if (NewElement)
	{
		UWorld* World = NewElement->GetWorld();
		if (World && World->WorldType == EWorldType::Editor)
		{
			if (bTransactional)
			{
				Modify();
			}
			CompElements.AddUnique(NewElement);
			bAdded = true;
		}
	}
	return bAdded;
}

bool UEditorCompElementContainer::Remove(ACompositingElement* Element, const bool bTransactional)
{
	bool bFound = false;
	for (int32 ElementIndex = CompElements.Num() - 1; ElementIndex >= 0 && !bFound; --ElementIndex)
	{
		if (CompElements[ElementIndex] == Element)
		{
			if (bTransactional)
			{
				Modify();
			}
			CompElements.RemoveAtSwap(ElementIndex);
			bFound = true;
		}
		else if (!CompElements[ElementIndex].IsValid())
		{
			// Clean up the list as we can, while iterating it
			CompElements.RemoveAtSwap(ElementIndex);
		}
	}

	return bFound;
}

bool UEditorCompElementContainer::Contains(ACompositingElement* Element) const
{
	return CompElements.Contains(Element);
}

void UEditorCompElementContainer::RebuildEditorElementsList()
{
	CompElements.Reset();

	for (TObjectIterator<ACompositingElement> ElementIt; ElementIt; ++ElementIt)
	{
		ULevel* ElementLevel = ElementIt->GetLevel();
		if (!ElementLevel || !FLevelUtils::IsLevelVisible(ElementLevel) || !FLevelUtils::IsLevelLoaded(ElementLevel))
		{
			continue;
		}

		UWorld* ElementWorld = ElementIt->GetWorld();
		if (ElementWorld && ElementWorld->WorldType == EWorldType::Editor)
		{
			TWeakObjectPtr<ACompositingElement> ElementPtr = *ElementIt;
			// Prevent pending-kill elements from being added to the list
			if (ElementPtr.IsValid())
			{
				CompElements.Add(ElementPtr);
			}
		}
	}
}

UWorld* UEditorCompElementContainer::GetWorld() const
{
	for (const TWeakObjectPtr<ACompositingElement>& Element : CompElements)
	{
		if (Element.IsValid())
		{
			UWorld* ElementWorld = Element->GetWorld();
			if (ElementWorld)
			{
				return ElementWorld;
			}
		}
	}

	return Super::GetWorld();
}
