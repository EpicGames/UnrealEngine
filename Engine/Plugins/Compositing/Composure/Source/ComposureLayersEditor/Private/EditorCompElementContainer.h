// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorCompElementContainer.generated.h"

class ACompositingElement;

/**
 * UObject for tracking our list of in-level composure actors - wrapped by a
 * UObject to mimic the UWorld::Layers property (hooks into undo/redo easily, etc.)
 */
UCLASS()
class UEditorCompElementContainer : public UObject
{

	GENERATED_BODY()

public:
	UEditorCompElementContainer();

	bool Add(ACompositingElement* NewElement, bool bTransactional = true);
	bool Remove(ACompositingElement* Element, bool bTransactional = true);
	bool Contains(ACompositingElement* Element) const;

	FORCEINLINE int32 Num() { return CompElements.Num(); }

	template <class PREDICATE_CLASS>
	void Sort(const PREDICATE_CLASS& Predicate)
	{
		CompElements.Sort(Predicate);
	}

	typedef TArray< TWeakObjectPtr<ACompositingElement> > FCompElementList;
	operator const FCompElementList&() { return CompElements; }

	void RebuildEditorElementsList();

public:
	//~ Begin UObject interface
	UWorld* GetWorld() const override;
	//~ End UObject interface

private:
	UPROPERTY()
	TArray< TWeakObjectPtr<ACompositingElement> > CompElements;

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	FORCEINLINE FCompElementList::RangedForIteratorType      begin()       { return CompElements.begin(); }
	FORCEINLINE FCompElementList::RangedForConstIteratorType begin() const { return CompElements.begin(); }
	FORCEINLINE FCompElementList::RangedForIteratorType      end  ()       { return CompElements.end();   }
	FORCEINLINE FCompElementList::RangedForConstIteratorType end  () const { return CompElements.end();   }
};
