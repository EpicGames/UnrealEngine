// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/World.h"

class ULevel;

struct FScopedWorldLevelContext
{
	FScopedWorldLevelContext(UWorld* InWorld, AActor* LevelContext)
		: World(InWorld)
	{
		if (InWorld)
		{
			LevelToRestore = InWorld->GetCurrentLevel();

			if (LevelContext)
			{
				InWorld->SetCurrentLevel(LevelContext->GetLevel());
			}
		}
	}

	FScopedWorldLevelContext(UWorld* InWorld, ULevel* Level)
		: World(InWorld)
	{
		if (InWorld)
		{
			LevelToRestore = InWorld->GetCurrentLevel();
			InWorld->SetCurrentLevel(Level);
		}
	}

	~FScopedWorldLevelContext()
	{
		if (World)
		{
			World->SetCurrentLevel(LevelToRestore);
		}
	}

private:
	UWorld* World = nullptr;
	ULevel* LevelToRestore = nullptr;
};
