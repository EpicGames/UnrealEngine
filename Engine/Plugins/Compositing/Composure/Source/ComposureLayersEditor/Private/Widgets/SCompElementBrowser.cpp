// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCompElementBrowser.h"
#include "CompElementEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "CompElementBrowser"

void SCompElementBrowser::Construct(const FArguments& InArgs)
{
	ICompElementEditorModule& CompEditorModule = FModuleManager::GetModuleChecked<ICompElementEditorModule>(TEXT("ComposureLayersEditor"));
	ElementCollectionViewModel = FCompElementCollectionViewModel::Create(CompEditorModule.GetCompElementManager().ToSharedRef(), GEditor);

	SearchBoxCompElementFilter = MakeShareable(new FCompElementTextFilter(FCompElementTextFilter::FItemToStringArray::CreateSP(this, &SCompElementBrowser::TransformElementToString)));
	ElementCollectionViewModel->AddFilter(SearchBoxCompElementFilter.ToSharedRef());

	ElementCollectionViewModel->OnRenameRequested().AddSP(this, &SCompElementBrowser::OnRenameRequested);

	ChildSlot
	[
		SNew(SBorder)
			.Padding(5)
			.BorderImage(FAppStyle::GetBrush("NoBrush"))
			.Content()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
				.AutoHeight()
			[
				SAssignNew(SearchBoxPtr, SSearchBox)
					.ToolTipText(LOCTEXT("FilterSearchToolTip", "Type here to search compositing elements"))
					.HintText(LOCTEXT("FilterSearchHint", "Search Compositing Elements"))
					.OnTextChanged(this, &SCompElementBrowser::OnFilterTextChanged)
			]
			+ SVerticalBox::Slot()
				.FillHeight(1.0f)
			[
				SAssignNew(ElementsView, SCompElementsView, ElementCollectionViewModel.ToSharedRef())
					.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
					.ConstructContextMenu(FOnContextMenuOpening::CreateSP(this, &SCompElementBrowser::ConstructElementContextMenu))
					.HighlightText(SearchBoxCompElementFilter.Get(), &FCompElementTextFilter::GetRawFilterText)
			]
		]
	];
}

void SCompElementBrowser::OnFilterTextChanged( const FText& InNewText )
{
	SearchBoxCompElementFilter->SetRawFilterText(InNewText);
	SearchBoxPtr->SetError(SearchBoxCompElementFilter->GetFilterErrorText());
}

#undef LOCTEXT_NAMESPACE
