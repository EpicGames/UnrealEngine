// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/TextFilter.h"
#include "Widgets/SCompElementsView.h"
#include "Widgets/SCompElementEdCommandsMenu.h"

class SSearchBox;

typedef TTextFilter<const TSharedPtr<FCompElementViewModel>&> FCompElementTextFilter;

/**
 *	
 */
class SCompElementBrowser : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCompElementBrowser) {}
	SLATE_END_ARGS()

	~SCompElementBrowser()
	{
		ElementCollectionViewModel->OnRenameRequested().RemoveAll(this);
		ElementCollectionViewModel->RemoveFilter(SearchBoxCompElementFilter.ToSharedRef());
	}

	/**
	 * Construct this widget. Called by the SNew() Slate macro.
	 *
	 * @param  InArgs	Declaration used by the SNew() macro to construct this widget
	 */
	void Construct(const FArguments& InArgs);

protected:
	//~ Begin SWidget interface
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override
	{
		TSharedPtr<FActorDragDropGraphEdOp> DragActorOp = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>();
		if(DragActorOp.IsValid())
		{
			DragActorOp->ResetToDefaultToolTip();
		}
	}
	//~ End SWidget interface

private:
	void TransformElementToString(const TSharedPtr<FCompElementViewModel>& Element, OUT TArray<FString>& OutSearchStrings) const
	{
		if (!Element.IsValid())
		{
			return;
		}

		OutSearchStrings.Add(Element->GetName());
	}
	
	/** Callback when elements want to be renamed */
	void OnRenameRequested()
	{
		ElementsView->RequestRenameOnSelectedElement();
	}

	TSharedPtr<SWidget> ConstructElementContextMenu()
	{
		return SNew(SCompElementEdCommandsMenu, ElementCollectionViewModel.ToSharedRef());
	}

	void OnFilterTextChanged(const FText& InNewText);

private:
	/** */
	TSharedPtr<SSearchBox> SearchBoxPtr;

	/**	 */
	TSharedPtr<FCompElementTextFilter> SearchBoxCompElementFilter;

	/**	 */
	TSharedPtr<FCompElementCollectionViewModel> ElementCollectionViewModel;

	/** The element view widget, displays all the compositing elements in the level */
	TSharedPtr<SCompElementsView> ElementsView;
};
