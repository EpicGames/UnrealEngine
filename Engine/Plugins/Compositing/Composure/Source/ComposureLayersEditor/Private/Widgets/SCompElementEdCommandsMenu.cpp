// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCompElementEdCommandsMenu.h"
#include "CompElementCollectionViewModel.h"
#include "CompElementEditorModule.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "CompElementEditorCommands.h"
#include "Framework/Commands/GenericCommands.h"

#define LOCTEXT_NAMESPACE "CompElementEdCommands"

void SCompElementEdCommandsMenu::Construct(const FArguments& InArgs, const TSharedRef<FCompElementCollectionViewModel> InViewModel)
{
	ViewModel = InViewModel;
	const FCompElementEditorCommands& Commands = FCompElementEditorCommands::Get();

	// Get all menu extenders for this context menu from the module
	const TArray<ICompElementEditorModule::FCompEditorMenuExtender>& MenuExtenderDelegates = ICompElementEditorModule::Get().GetEditorMenuExtendersList();

	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute(ViewModel->GetCommandList()));
		}
	}
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);

	// Build up the menu
	FMenuBuilder MenuBuilder(InArgs._CloseWindowAfterMenuSelection, ViewModel->GetCommandList(), MenuExtender);
	{
		MenuBuilder.BeginSection("CompsCreate", LOCTEXT("CompsMenuHeader", "Comp Shots"));
		{
			MenuBuilder.AddMenuEntry(Commands.CreateEmptyComp);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("CompElements", LOCTEXT("ElementsMenuHeader", "Layer Elements"));
		{
			MenuBuilder.AddMenuEntry(Commands.CreateNewElement);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("Edit");

		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut, "CutComp", LOCTEXT("CutComp", "Cut"), LOCTEXT("CutCompToolTip", "Cuts the selected comp actors."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy, "CopyComp", LOCTEXT("CopyComp", "Copy"), LOCTEXT("CopyCompToolTip", "Copies the selected comp actors."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste, "PasteComp", LOCTEXT("PasteComp", "Paste"), LOCTEXT("PasteCompToolTop", "Adds the copied comp actors to the level."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate, "DuplicateComp", LOCTEXT("DuplicateComp", "Duplicate"), LOCTEXT("DuplicateCompToolTip", "Duplicates the selected comp actors."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, "DeleteComp", LOCTEXT("DeleteComp", "Delete"), LOCTEXT("DeleteCompToolTip", "Deletes the selected comp actors from the level."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, "RenameComp", LOCTEXT("RenameComp", "Rename"), LOCTEXT("RenameCompToolTip", "Renames the selected comp actors."));

		MenuBuilder.EndSection();

		MenuBuilder.AddMenuEntry(Commands.OpenElementPreview);
	}

	ChildSlot
		[
			MenuBuilder.MakeWidget()
		];
}


#undef LOCTEXT_NAMESPACE
