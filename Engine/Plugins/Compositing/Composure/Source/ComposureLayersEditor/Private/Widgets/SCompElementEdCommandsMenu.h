// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"

class FCompElementCollectionViewModel;

class SCompElementEdCommandsMenu : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCompElementEdCommandsMenu)
		: _CloseWindowAfterMenuSelection(true) {}

		SLATE_ARGUMENT(bool, CloseWindowAfterMenuSelection)
	SLATE_END_ARGS()

	/** Construct this widget. Called by the SNew() Slate macro. */
	void Construct(const FArguments& InArgs, const TSharedRef<FCompElementCollectionViewModel> InViewModel);

private:
	/** The UI logic of the panel that is not Slate specific */
	TSharedPtr<FCompElementCollectionViewModel> ViewModel;
};

