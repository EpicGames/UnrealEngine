// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCompElementPickerWindow.h"
#include "Editor.h" // for GEditor
#include "CompositingElement.h" // for ETargetUsageFlags
#include "Layout/WidgetPath.h"
#include "Widgets/SCompElementPreviewDialog.h"
#include "ComposureEditorStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "BlueprintMaterialTextureNodesBPLibrary.h"
#include "CompElementEditorCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Widgets/SCompElementPreviewPane.h"

#define LOCTEXT_NAMESPACE "CompElementPickerWindow"

namespace CompElementPickerWindow_Impl
{
	static TWeakPtr<SWindow> OpenWindow;
	static FSimpleDelegate   OpenWindowCancelCallback;

	static void OnWindowClosed(const TSharedRef<SWindow>& Window);
}

static void CompElementPickerWindow_Impl::OnWindowClosed(const TSharedRef<SWindow>& Window)
{
	if (OpenWindow.IsValid() && OpenWindow.Pin() == Window)
	{
		OpenWindow.Reset();
	}
}

static bool CanAlwaysExecute()
{
	return true;
}

/* SPickerButton
 *****************************************************************************/

enum class EPickingState : uint8
{
	Released,
	PointSampling,
	Averaging,
	PickAndAccept,
};
DECLARE_DELEGATE_TwoParams(FOnPixelSampled, const FVector2D&, EPickingState)

class SPickerButton : public SButton
{
public:
	FOnPixelSampled OnPixelPicked;

public:
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (PickingMode == EPickingState::Released)
		{
			PickingMode = EPickingState::PointSampling;

			PickedPixelUV = CalcSampleUVPoint(MyGeometry, MouseEvent.GetScreenSpacePosition());
			OnPixelPicked.ExecuteIfBound(PickedPixelUV, PickingMode);
		}
		if (!MouseEvent.IsControlDown())
		{
			PickingMode = EPickingState::Averaging;
		}
		return SButton::OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{	
		//ensure(IsPressed() || PickingMode == EPickingState::Released);

		if (PickingMode == EPickingState::PointSampling)
		{
			PickedPixelUV = CalcSampleUVPoint(MyGeometry, MouseEvent.GetScreenSpacePosition());
			OnPixelPicked.ExecuteIfBound(PickedPixelUV, PickingMode);
		}
		PickingMode = EPickingState::Released;

		return SButton::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (IsPressed())
		{
			ensure(PickingMode != EPickingState::Released);
			PickingMode = MouseEvent.IsControlDown() ? EPickingState::PointSampling : EPickingState::Averaging;

			PickedPixelUV = CalcSampleUVPoint(MyGeometry, MouseEvent.GetScreenSpacePosition());
			if (PickedPixelUV.GetMax() <= 1.0f)
			{
				OnPixelPicked.ExecuteIfBound(PickedPixelUV, PickingMode);
			}
		}
		return SButton::OnMouseMove(MyGeometry, MouseEvent);
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		PickedPixelUV = CalcSampleUVPoint(MyGeometry, MouseEvent.GetScreenSpacePosition());
		OnPixelPicked.ExecuteIfBound(PickedPixelUV, EPickingState::PickAndAccept);

		return SButton::OnMouseButtonDoubleClick(MyGeometry, MouseEvent);
	}

private:
	FVector2D CalcSampleUVPoint(const FGeometry& Geometry, const FVector2D& ScreenSpacePos)
	{
		return Geometry.AbsoluteToLocal(ScreenSpacePos) / Geometry.GetLocalSize();
	}

	FVector2D PickedPixelUV;
	EPickingState PickingMode = EPickingState::Released;
};

/* SCompElementColorPickerDialog
 *****************************************************************************/

class SCompElementColorPickerDialog : public SCompElementPreviewDialog
{
public:
	SCompElementColorPickerDialog();

	SLATE_BEGIN_ARGS(SCompElementColorPickerDialog)	{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(TWeakUIntrfacePtr<ICompImageColorPickerInterface>, PickerTarget)
		SLATE_ARGUMENT(FFreezeFrameControlHandle, FreezeFrameControlHandle)
		SLATE_ARGUMENT(bool, AverageColorOnDrag)
		SLATE_EVENT(FColorPickedEventHandler, OnColorPicked)
		SLATE_EVENT(FOnClicked, OnClicked)
		SLATE_EVENT(FColorPickedEventHandler, OnAccept)
		SLATE_EVENT(FSimpleDelegate, OnCancel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetPreviewing(const bool bShowPreview = true);
	void ResetSampling();
	void ToggleFreezeFraming();

private:
	//~ Begin SCompElementPreviewDialog interface
	virtual TSharedRef<SWidget> GeneratePreviewContent() override;
	virtual void ExtendMenuOverlay(TSharedRef<SHorizontalBox> MenuOverlay) override;
	//~ End SCompElementPreviewDialog interface

private:
	void OnPixelPicked(const FVector2D& UVCoord, EPickingState PickingMode);
	FReply OnClick();
	FReply OnAcceptClicked();
	FReply OnCancelClicked();
	EVisibility GetAcceptButtonVisibility() const;
	FLinearColor GetPickedColor() const;
	EVisibility GetAveragingReadoutVisibility() const;
	FReply OnToggleInputFreeze();
	const FSlateBrush* GetFreezeFrameToggleBrush() const;
	bool IsInputFrozen() const;

	void RefreshDisplayImage();
	
private:
	TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget;
	TSharedPtr<SPickerButton> PickerButton;
	FColorPickedEventHandler OnColorPicked;
	FOnClicked OnClicked;
	FColorPickedEventHandler OnAccept;
	FSimpleDelegate OnCancel;
	FFreezeFrameControlHandle FreezeFrameControlHandle;

	int32 PickedSamples = 0;
	FLinearColor PickedColor = FLinearColor::Black;
	FVector2D LastPickUv = FVector2D(-1.f, -1.f);

	TSharedRef<FUICommandList> CommandList;
	bool bShowPreview = false;

	bool bAverageColorOnDrag = true;
};

SCompElementColorPickerDialog::SCompElementColorPickerDialog()
	: CommandList(MakeShareable(new FUICommandList))
{}

void SCompElementColorPickerDialog::Construct(const FArguments& InArgs)
{
	PickerTarget = InArgs._PickerTarget;
	FreezeFrameControlHandle = InArgs._FreezeFrameControlHandle;
	OnColorPicked = InArgs._OnColorPicked;
	OnClicked = InArgs._OnClicked;
	OnAccept = InArgs._OnAccept;
	OnCancel = InArgs._OnCancel;
	bAverageColorOnDrag = InArgs._AverageColorOnDrag;

	SCompElementPreviewDialog::Construct
	(
		SCompElementPreviewDialog::FArguments()
			.ParentWindow(InArgs._ParentWindow)
			.PreviewTarget(InArgs._PickerTarget)
	);

	if (ImagePane.IsValid())
	{
		ImagePane->SetOnRedraw(FSimpleDelegate::CreateSP(this, &SCompElementColorPickerDialog::RefreshDisplayImage));
	}
}

void SCompElementColorPickerDialog::SetPreviewing(const bool bInShowPreview)
{
	bShowPreview = bInShowPreview;
}

void SCompElementColorPickerDialog::ResetSampling()
{
	PickedColor = FLinearColor::Black;
	PickedSamples = 0;
}

void SCompElementColorPickerDialog::ToggleFreezeFraming()
{
	if (PickerTarget.IsValid())
	{
		FCompFreezeFrameController* FreezeFrameController = PickerTarget->GetFreezeFrameController();
		if (FreezeFrameController)
		{
			if (IsInputFrozen())
			{
				FreezeFrameController->ClearFreezeFlags(FreezeFrameControlHandle);
			}
			else
			{
				FreezeFrameController->SetFreezeFlags(ETargetUsageFlags::USAGE_Input, /*bClearOthers =*/true, FreezeFrameControlHandle);
			}
		}
	}
}

TSharedRef<SWidget> SCompElementColorPickerDialog::GeneratePreviewContent()
{
	TSharedPtr<SVerticalBox> BodyContent;

	SAssignNew(BodyContent, SVerticalBox)
	+SVerticalBox::Slot()
		.AutoHeight()
	[
		SAssignNew(PickerButton, SPickerButton)
			.ButtonStyle(FComposureEditorStyle::Get(), "ColorPickerPreviewButton")
			.ContentPadding(0.0f)
			.OnClicked(this, &SCompElementColorPickerDialog::OnClick)
			.Cursor(EMouseCursor::EyeDropper)
			.Content()
			[
				SCompElementPreviewDialog::GeneratePreviewContent()
			]
	]
	+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(8.f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 4.f, 0.f)
		[
			SNew(SBox)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(LOCTEXT("AveragingMsg", "Average:"))
					.Visibility(this, &SCompElementColorPickerDialog::GetAveragingReadoutVisibility)
			]
		]
		+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0)
		[
			SNew(SBox)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
			[
				SNew(SColorBlock)
					.Color(this, &SCompElementColorPickerDialog::GetPickedColor)
					.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
					.Size(FVector2D(16.0f, 16.0f))
					.Visibility(this, &SCompElementColorPickerDialog::GetAcceptButtonVisibility)
			]
		]
		+SHorizontalBox::Slot()
			.AutoWidth()
		[
			SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			+SUniformGridPanel::Slot(0, 0)
			[
				SNew(SButton)
					.Text(LOCTEXT("AcceptColorSample", "Accept"))
					.HAlign(HAlign_Center)
					.Visibility(this, &SCompElementColorPickerDialog::GetAcceptButtonVisibility)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SCompElementColorPickerDialog::OnAcceptClicked)
					.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
					.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")			
			]
			+SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
					.Text(LOCTEXT("CancelColorSample", "Cancel"))
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SCompElementColorPickerDialog::OnCancelClicked)
					.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
					.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
			]
		]
	];
	
	PickerButton->OnPixelPicked.BindRaw(this, &SCompElementColorPickerDialog::OnPixelPicked);

	return BodyContent.ToSharedRef();
}

void SCompElementColorPickerDialog::ExtendMenuOverlay(TSharedRef<SHorizontalBox> MenuBar) 
{
	if (PickerTarget.IsValid() && PickerTarget->GetFreezeFrameController())
	{
		MenuBar->AddSlot()
			.AutoWidth()
		[
			SNew(SButton)
				.ContentPadding(0)
				.ButtonStyle(FAppStyle::Get(), "ToggleButton")
				.ToolTipText(LOCTEXT("FreezeToggleTooltip", "Toggle Input Freeze"))
				.OnClicked(this, &SCompElementColorPickerDialog::OnToggleInputFreeze)
				.Cursor(EMouseCursor::Default)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Content()
				[
					SNew(SImage)
					.Image(this, &SCompElementColorPickerDialog::GetFreezeFrameToggleBrush)
				]
		];
	}
}

void SCompElementColorPickerDialog::OnPixelPicked(const FVector2D& UVCoord, EPickingState PickingMode)
{
	ensure(PickingMode != EPickingState::Released);
	LastPickUv = UVCoord;

	if (PickerTarget.IsValid())
	{
		UTextureRenderTarget2D* PickerSourceImg = PickerTarget->GetColorPickerTarget();
		if (PickerSourceImg)
		{
			ensure(PickingMode != EPickingState::Averaging || PickedSamples > 0);

			FLinearColor NewSample = UBlueprintMaterialTextureNodesBPLibrary::RenderTarget_SampleUV_EditorOnly(PickerSourceImg, UVCoord);
			if (PickingMode == EPickingState::Averaging && bAverageColorOnDrag)
			{
				// use a FVector4 to prevent FLinearColorfrom clamping values
				FVector4f ColorVec = PickedColor;
				ColorVec *= PickedSamples;
				ColorVec += NewSample;
				ColorVec *= 1.f / ++PickedSamples;

				PickedColor = FLinearColor(ColorVec);
			}
			else
			{
				PickedColor = NewSample;
				PickedSamples = 1;
			}
		}

		if (PickingMode == EPickingState::PickAndAccept)
		{
			OnColorPicked.ExecuteIfBound(UVCoord, PickedColor, /*bInteractive =*/false);
			OnAcceptClicked();
		}
		else
		{
			OnColorPicked.ExecuteIfBound(UVCoord, PickedColor, /*bInteractive =*/true);
		}
	}
}

FReply SCompElementColorPickerDialog::OnClick()
{
	if (OnClicked.IsBound())
	{
		OnClicked.Execute();
	}
	return FReply::Handled();
}

FReply SCompElementColorPickerDialog::OnAcceptClicked()
{
	if (OnAccept.IsBound())
	{
		OnAccept.Execute(LastPickUv, PickedColor, /*bInteractive =*/false);
	}
	return FReply::Handled();
}

FReply SCompElementColorPickerDialog::OnCancelClicked()
{
	if (OnCancel.IsBound())
	{
		OnCancel.Execute();
	}
	return FReply::Handled();
}

EVisibility SCompElementColorPickerDialog::GetAcceptButtonVisibility() const
{
	return (PickedSamples > 0) ? EVisibility::Visible : EVisibility::Hidden;
}

FLinearColor SCompElementColorPickerDialog::GetPickedColor() const
{
	return PickedColor;
}

EVisibility SCompElementColorPickerDialog::GetAveragingReadoutVisibility() const
{
	EVisibility MyVisibility = GetAcceptButtonVisibility();
	if (MyVisibility == EVisibility::Visible)
	{
		MyVisibility = (PickedSamples > 1) ? EVisibility::Visible : EVisibility::Hidden;
	}
	return MyVisibility;
}

FReply SCompElementColorPickerDialog::OnToggleInputFreeze()
{
	ToggleFreezeFraming();
	return FReply::Handled();
}

const FSlateBrush* SCompElementColorPickerDialog::GetFreezeFrameToggleBrush() const
{
	if (IsInputFrozen())
	{
		return IsHovered() ? FComposureEditorStyle::Get().GetBrush("ComposureTree.FrameFrozenHighlightIcon16x") :
			FComposureEditorStyle::Get().GetBrush("ComposureTree.FrameFrozenIcon16x");
	}
	else
	{
		return IsHovered() ? FComposureEditorStyle::Get().GetBrush("ComposureTree.NoFreezeFrameHighlightIcon16x") :
			FComposureEditorStyle::Get().GetBrush("ComposureTree.NoFreezeFrameIcon16x");
	}
}

bool SCompElementColorPickerDialog::IsInputFrozen() const
{
	if (PickerTarget.IsValid())
	{
		if (FCompFreezeFrameController* FreezeFrameController = PickerTarget->GetFreezeFrameController())
		{
			return FreezeFrameController->HasAnyFlags(ETargetUsageFlags::USAGE_Input);
		}
		return false;
	}
	return true;
}

void SCompElementColorPickerDialog::RefreshDisplayImage()
{
	if (!bShowPreview && PickerTarget.IsValid() && ImagePane.IsValid())
	{
		UTexture* PickerDisplayImage = PickerTarget->GetColorPickerDisplayImage();
		if (PickerDisplayImage)
		{
			ImagePane->SetDisplayImage(PickerDisplayImage);
		}
	}
}

/* SCompElementPickerWindow
 *****************************************************************************/

TSharedPtr<SWindow> SCompElementPickerWindow::Open(const FCompElementColorPickerArgs& PickerArgs)
{	
	using namespace CompElementPickerWindow_Impl;

	if (OpenWindow.IsValid())
	{
		OpenWindowCancelCallback.ExecuteIfBound();
		OpenWindow.Pin()->RequestDestroyWindow();
		OpenWindow.Reset();
	}

	TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget = PickerArgs.PickerTarget;

	ETargetUsageFlags OldFreezeFlags = ETargetUsageFlags::USAGE_None;
	FFreezeFrameControlHandle FreezeFrameControlHandle;
	if (PickerTarget.IsValid())
	{
		FCompFreezeFrameController* FreezeFrameController = PickerTarget->GetFreezeFrameController();
		if (FreezeFrameController)
		{
			OldFreezeFlags = FreezeFrameController->GetFreezeFlags();
			FreezeFrameController->SetFreezeFlags(ETargetUsageFlags::USAGE_Input, /*bClearOthers =*/true);
			FreezeFrameControlHandle = FreezeFrameController->Lock();
		}
	}

	TSharedRef<SWindow> PickerWindow = SNew(SCompElementPickerWindow)
		.PickerTarget(PickerTarget)
		.WindowTitle(PickerArgs.WindowTitle)
		.FreezeFrameControlHandle(FreezeFrameControlHandle)
		.OnColorPicked(PickerArgs.OnColorPicked)
		.OnPickerCanceled(PickerArgs.OnColorPickerCanceled)
		.AverageColorOnDrag(PickerArgs.bAverageColorOnDrag);

	OpenWindow = PickerWindow;
	OpenWindowCancelCallback = PickerArgs.OnColorPickerCanceled;

	PickerWindow->SetOnWindowClosed(FOnWindowClosed::CreateLambda([PickerTarget, FreezeFrameControlHandle, OldFreezeFlags](const TSharedRef<SWindow>& Window)
		{
			if (PickerTarget.IsValid())
			{
				FCompFreezeFrameController* FreezeFrameController = PickerTarget->GetFreezeFrameController();
				if (FreezeFrameController && FreezeFrameControlHandle.IsValid())
				{
					FreezeFrameController->Unlock(FreezeFrameControlHandle);
					FreezeFrameController->SetFreezeFlags(OldFreezeFlags, /*bClearOthers =*/true);
				}
			}
		}));

	if (PickerArgs.ParentWidget.IsValid())
	{
		FWidgetPath WidgetPath;
		FSlateApplication::Get().GeneratePathToWidgetChecked(PickerArgs.ParentWidget.ToSharedRef(), WidgetPath);
		PickerWindow = FSlateApplication::Get().AddWindowAsNativeChild(PickerWindow, WidgetPath.GetWindow());
	}
	else
	{
		GEditor->EditorAddModalWindow(PickerWindow);
	}

	return PickerWindow;
}

SCompElementPickerWindow::SCompElementPickerWindow()
	: CommandList(MakeShareable(new FUICommandList))
{}

void SCompElementPickerWindow::Construct(const FArguments& InArgs)
{
	SWindow::Construct
	(
		SWindow::FArguments()
			.Title(InArgs._WindowTitle.IsEmpty() ? LOCTEXT("PickAColorTitle", "Pick a color") : InArgs._WindowTitle)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.SizingRule(ESizingRule::Autosized)
			.ClientSize(FVector2D(0.f, 300.f))
			.SupportsMaximize(false)
			.SupportsMinimize(false)
	);		

	TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget; 
	PickerTarget = InArgs._PickerTarget;

	FColorPickedEventHandler OnColorPicked = InArgs._OnColorPicked;
	OnPickerCanceled = InArgs._OnPickerCanceled;

	TSharedRef<SCompElementColorPickerDialog> PickerDialog = SAssignNew(PickerContents, SCompElementColorPickerDialog)
		.ParentWindow(SharedThis(this))
		.PickerTarget(PickerTarget)
		.FreezeFrameControlHandle(InArgs._FreezeFrameControlHandle)
		.OnColorPicked(OnColorPicked)
		.AverageColorOnDrag(InArgs._AverageColorOnDrag)
		.OnAccept_Lambda([OnColorPicked, this](const FVector2D& PickedUV, const FLinearColor& PickedColor, bool bInteractive)
		{
			OnColorPicked.ExecuteIfBound(PickedUV, PickedColor, bInteractive);
			RequestDestroyWindow();

		})
		.OnCancel_Lambda([this]()
		{
			OnPickerCanceled.ExecuteIfBound();
			RequestDestroyWindow();
		});

	SetContent(PickerDialog);

	BindCommands();
}

FReply SCompElementPickerWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	TGuardValue<bool> PressGuard(bProcessingKeyDown, true);
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	else
	{
		return SWindow::OnKeyDown(MyGeometry, InKeyEvent);
	}
}

FReply SCompElementPickerWindow::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	else
	{
		return SWindow::OnKeyDown(MyGeometry, InKeyEvent);
	}
}

void SCompElementPickerWindow::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	while (PressedCmds.Num() > 0)
	{
		TSharedPtr<const FUICommandInfo> Cmd = *PressedCmds.CreateIterator();

		const FUIAction* Action = CommandList->GetActionForCommand(Cmd);
		if (Action && Action->CanExecute())
		{
			Action->Execute();
		}
		else
		{
			PressedCmds.Remove(Cmd);
		}
	}

	SWindow::OnFocusLost(InFocusEvent);
}

void SCompElementPickerWindow::BindCommands()
{
	const FCompElementEditorCommands& Commands = FCompElementEditorCommands::Get();
	FUICommandList& ActionList = *CommandList;

	auto MapKeyDownOnlyAction = [&ActionList, this](const TSharedPtr<const FUICommandInfo> CmdInfo, FExecuteAction ExecuteAction, FCanExecuteAction CanExectue)
	{
		ActionList.MapAction(CmdInfo,
			FExecuteAction::CreateLambda([this, ExecuteAction]()
			{
				if(bProcessingKeyDown)
				{
					ExecuteAction.ExecuteIfBound();
				}
			}),
			FCanExecuteAction::CreateLambda([this, CanExectue]()->bool
			{
				if (!bProcessingKeyDown)
				{
					return false;
				}
				else if (CanExectue.IsBound())
				{
					return CanExectue.Execute();
				}
				return true;
			}) 
		);
	};

	auto MapKeyPressReleaseAction = [&ActionList, this](const TSharedPtr<const FUICommandInfo> CmdInfo, FExecuteAction OnPress, FExecuteAction OnRelease, FCanExecuteAction CanExectue)
	{
		ActionList.MapAction(CmdInfo,
			FExecuteAction::CreateLambda([this, OnPress, OnRelease, CmdInfo]()
			{
				if (bProcessingKeyDown && ensure(!PressedCmds.Contains(CmdInfo)))
				{
 					PressedCmds.Add(CmdInfo);
					OnPress.ExecuteIfBound();
				}
				else if (PressedCmds.Contains(CmdInfo))
				{
					PressedCmds.Remove(CmdInfo);
					OnRelease.ExecuteIfBound();
				}
			}),
			FCanExecuteAction::CreateLambda([this, CanExectue, CmdInfo]()->bool
			{
				if (!bProcessingKeyDown && !PressedCmds.Contains(CmdInfo))
				{
					return false;
				}
				else if (CanExectue.IsBound())
				{
					return CanExectue.Execute();
				}
				return true;
			})
		);
	};

	MapKeyPressReleaseAction( Commands.OpenElementPreview,
		FExecuteAction::CreateSP(this, &SCompElementPickerWindow::OnPreviewPressed),
		FExecuteAction::CreateSP(this, &SCompElementPickerWindow::OnPreviewReleased),
		FCanExecuteAction::CreateStatic(&CanAlwaysExecute) );

	MapKeyDownOnlyAction( Commands.ResetColorPicker, 
		FExecuteAction::CreateSP(this, &SCompElementPickerWindow::OnResetPicking),
		FCanExecuteAction::CreateStatic(&CanAlwaysExecute) );

	MapKeyDownOnlyAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SCompElementPickerWindow::OnResetPicking),
		FCanExecuteAction::CreateStatic(&CanAlwaysExecute));

	MapKeyDownOnlyAction(Commands.FreezeFrame,
		FExecuteAction::CreateSP(this, &SCompElementPickerWindow::OnToggleFreezeFrame),
		FCanExecuteAction::CreateStatic(&CanAlwaysExecute));
}

void SCompElementPickerWindow::OnPreviewPressed()
{
	PickerContents->SetPreviewing(true);
}

void SCompElementPickerWindow::OnPreviewReleased()
{
	PickerContents->SetPreviewing(false);
}

void SCompElementPickerWindow::OnResetPicking()
{
	PickerContents->ResetSampling();
	OnPickerCanceled.ExecuteIfBound();
}

void SCompElementPickerWindow::OnToggleFreezeFrame()
{
	PickerContents->ToggleFreezeFraming();
}

#undef LOCTEXT_NAMESPACE
