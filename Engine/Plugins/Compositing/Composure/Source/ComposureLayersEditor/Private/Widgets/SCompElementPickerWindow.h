// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSupport/CompImageColorPickerInterface.h"
#include "EditorSupport/WeakUInterfacePtr.h"
#include "Widgets/SWindow.h"
#include "EditorSupport/CompFreezeFrameController.h" // for FFreezeFrameControlHandle

class FUICommandInfo;
class FUICommandList;

class ICompImageColorPickerInterface;
class SWidget;

DECLARE_DELEGATE_ThreeParams(FColorPickedEventHandler, const FVector2D& /*PickedUV*/, const FLinearColor& /*PickedColor*/, bool /*bInteractive*/);

struct FCompElementColorPickerArgs
{
	TWeakUIntrfacePtr<ICompImageColorPickerInterface> PickerTarget;
	FText WindowTitle;
	
	FColorPickedEventHandler OnColorPicked;
	FSimpleDelegate OnColorPickerCanceled;

	TSharedPtr<SWidget> ParentWidget;

	bool bAverageColorOnDrag = true;
};

/* SCompElementPickerWindow
 *****************************************************************************/

class SCompElementColorPickerDialog;

class SCompElementPickerWindow : public SWindow
{
public: 
	static TSharedPtr<SWindow> Open(const FCompElementColorPickerArgs& PickerArgs);

public:
	SCompElementPickerWindow();

	SLATE_BEGIN_ARGS(SCompElementPickerWindow)	{}
		SLATE_ARGUMENT(TWeakUIntrfacePtr<ICompImageColorPickerInterface>, PickerTarget)
		SLATE_ARGUMENT(FFreezeFrameControlHandle, FreezeFrameControlHandle)
		SLATE_ARGUMENT(bool, AverageColorOnDrag)
		SLATE_EVENT(FColorPickedEventHandler, OnColorPicked)
		SLATE_EVENT(FSimpleDelegate, OnPickerCanceled)
		SLATE_ARGUMENT(FText, WindowTitle)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

public:
	//~ Begin SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	//~ End SWidget interface

private:
	void BindCommands();

	void OnPreviewPressed();
	void OnPreviewReleased();

	void OnResetPicking();
	void OnToggleFreezeFrame();
	
private:
	TSharedRef<FUICommandList> CommandList;
	bool bProcessingKeyDown = false;
	TSet< TSharedPtr<const FUICommandInfo> > PressedCmds;

	TSharedPtr<SCompElementColorPickerDialog> PickerContents;
	FSimpleDelegate OnPickerCanceled;
};
