// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SCompElementPreviewDialog.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "CompElementEditorCommands.h"
#include "BlueprintMaterialTextureNodesBPLibrary.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Widgets/SCompElementPreviewPane.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCompElementPreviewDialog"

/* SCompElementPreviewDialog
 *****************************************************************************/

enum class EChannelPresets : uint8
{
	RGB,
	RGBA,
	A,
	None
};

TSharedRef<SWindow> SCompElementPreviewDialog::OpenPreviewWindow(TWeakUIntrfacePtr<ICompEditorImagePreviewInterface> PreviewTarget, TSharedPtr<SWidget> ParentWidget, const FText& WindowTitle)
{
	TSharedRef<SWindow> PreviewWindow = SNew(SWindow)
		.Title(WindowTitle.IsEmpty() ? LOCTEXT("PreviewWindowTitle", "Preview") : WindowTitle)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(1280.f, 720.f))
		.SupportsMaximize(true)
		.SupportsMinimize(true);

	TSharedRef<SCompElementPreviewDialog> PickerDialog = SNew(SCompElementPreviewDialog)
		.ParentWindow(PreviewWindow)
		.PreviewTarget(PreviewTarget);
		
	PreviewWindow->SetContent(PickerDialog);

	if (ParentWidget.IsValid())
	{
		FWidgetPath WidgetPath;
		FSlateApplication::Get().GeneratePathToWidgetChecked(ParentWidget.ToSharedRef(), WidgetPath);
		PreviewWindow = FSlateApplication::Get().AddWindowAsNativeChild(PreviewWindow, WidgetPath.GetWindow());
	}
	else
	{
		FSlateApplication::Get().AddWindow(PreviewWindow);
	}

	return PreviewWindow;
}

SCompElementPreviewDialog::SCompElementPreviewDialog()
	: CommandList(MakeShareable(new FUICommandList))
	, ChannelPreset(EChannelPresets::RGB)
{}

SCompElementPreviewDialog::~SCompElementPreviewDialog()
{
	CommandList = nullptr;
}

void SCompElementPreviewDialog::Construct(const FArguments& InArgs)
{
	WeakParentWindow = InArgs._ParentWindow;
	PreviewTarget    = InArgs._PreviewTarget;

	ChildSlot
	[
		SNew(SBorder)
			.Padding(FMargin(2.f))
			.Content()
		[
			GeneratePreviewContent()	
		]
	];

	BindCommands();
}

FReply SCompElementPreviewDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SCompElementPreviewDialog::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FVector2D PixelUnderMouse = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()) / MyGeometry.GetLocalSize();

	if (PreviewTarget.IsValid())
	{
		UTextureRenderTarget2D* SourceImage = Cast<UTextureRenderTarget2D>(PreviewTarget->GetEditorPreviewImage());
		if (SourceImage)
		{
			ColorUnderMouse = UBlueprintMaterialTextureNodesBPLibrary::RenderTarget_SampleUV_EditorOnly(SourceImage, PixelUnderMouse);
		}
	}
	
	return FReply::Unhandled();
}


void SCompElementPreviewDialog::BindCommands()
{

	const FCompElementEditorCommands& Commands = FCompElementEditorCommands::Get();
	FUICommandList& ActionList = *CommandList;

	ActionList.MapAction(Commands.CycleChannelPresets,
		FExecuteAction::CreateSP(this, &SCompElementPreviewDialog::OnCycleChannelPresets),
		FCanExecuteAction::CreateStatic(&CanAlwaysExecute)
	);

	ActionList.MapAction(Commands.SetChannelRed,
		FExecuteAction::CreateSP(this, &SCompElementPreviewDialog::SetChannelRed),
		FCanExecuteAction::CreateStatic(&CanAlwaysExecute)
	);

	ActionList.MapAction(Commands.SetChannelGreen,
		FExecuteAction::CreateSP(this, &SCompElementPreviewDialog::SetChannelGreen),
		FCanExecuteAction::CreateStatic(&CanAlwaysExecute)
	);

	ActionList.MapAction(Commands.SetChannelBlue,
		FExecuteAction::CreateSP(this, &SCompElementPreviewDialog::SetChannelBlue),
		FCanExecuteAction::CreateStatic(&CanAlwaysExecute)
	);

	ActionList.MapAction(Commands.SetChannelAlpha,
		FExecuteAction::CreateSP(this, &SCompElementPreviewDialog::SetChannelAlpha),
		FCanExecuteAction::CreateStatic(&CanAlwaysExecute)
	);

}

TSharedRef<SWidget> SCompElementPreviewDialog::GeneratePreviewContent()
{
	return SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SBox)
			.Padding(FMargin(0.f))
			.MaxDesiredHeight(720.f)
			.MaxDesiredWidth(1280.f)
			.Content()
			[
				SAssignNew(ImagePane, SCompElementPreviewPane)
					.PreviewTarget(PreviewTarget)
					.MenuOverlayExtender(this, &SCompElementPreviewDialog::ExtendMenuOverlay)
					.OnColorMaskChanged(this, &SCompElementPreviewDialog::SyncColorMaskPreset)
			]
		]
		+ SOverlay::Slot()
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
		[
			GenerateHoveredColorOverlay()
		];

}

TSharedRef<SHorizontalBox> SCompElementPreviewDialog::GenerateHoveredColorOverlay()
{
	const FMargin TextPadding(10.0f, 10.0f);

	FNumberFormattingOptions FormatOptions;
	FormatOptions.MinimumFractionalDigits = 6;
	FormatOptions.MaximumFractionalDigits = 6;
	FormatOptions.MinimumIntegralDigits = 1;
	FormatOptions.MaximumIntegralDigits = 1;

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(TextPadding)
		[
			SNew(STextBlock)
			.Text_Lambda([this, FormatOptions]() -> FText {
				return FText::AsNumber(ColorUnderMouse.R, &FormatOptions);
			})
			.ColorAndOpacity(FLinearColor(1.0f, 0.0f, 0.0f))
		]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(TextPadding)
		[
			SNew(STextBlock)
			.Text_Lambda([this, FormatOptions]() -> FText {
				return FText::AsNumber(ColorUnderMouse.G, &FormatOptions);
			}).ColorAndOpacity(FLinearColor(0.0f, 1.0f, 0.0f))
		]
	+ SHorizontalBox::Slot()
		.Padding(TextPadding)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text_Lambda([this, FormatOptions]() -> FText {
				return FText::AsNumber(ColorUnderMouse.B, &FormatOptions);
			}).ColorAndOpacity(FLinearColor(0.0f, 0.0f, 1.0f))
		]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(TextPadding)
		[
			SNew(STextBlock)
			.Text_Lambda([this, FormatOptions]() -> FText {
				return FText::AsNumber(ColorUnderMouse.A, &FormatOptions);
			}).ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
		];

}

void SCompElementPreviewDialog::OnCycleChannelPresets()
{
	FLinearColor ColorMask(1.f, 1.f, 1.f, 0.f);
	switch (ChannelPreset)
	{
		case EChannelPresets::RGB:
			ChannelPreset = EChannelPresets::RGBA;
			ColorMask.R = 1;
			ColorMask.G = 1;
			ColorMask.B = 1;
			ColorMask.A = 1;
		break;
		case EChannelPresets::RGBA:
			ChannelPreset = EChannelPresets::A;
			ColorMask.R = 0;
			ColorMask.G = 0;
			ColorMask.B = 0;
			ColorMask.A = 1;
			break;
		case EChannelPresets::A:
		case EChannelPresets::None:
			ChannelPreset = EChannelPresets::RGB;
			ColorMask.R = 1;
			ColorMask.G = 1;
			ColorMask.B = 1;
			ColorMask.A = 0;
		break;
	}

	if (ImagePane.IsValid())
	{
		ImagePane->SetPreviewColorMask(ColorMask);
	}
}

void SCompElementPreviewDialog::SetChannelRed()
{
	FLinearColor ColorMask(1,1,1,0);
	if (ImagePane.IsValid())
	{
		ColorMask = ImagePane->GetPreviewColorMask();
	}

	if (ColorMask.R == 1 && ColorMask.G == 0 && ColorMask.B == 0 && ColorMask.A == 0)
	{
		ColorMask = FLinearColor(1, 1, 1, 0);
		ChannelPreset = EChannelPresets::RGB;
	}
	else
	{
		ColorMask = FLinearColor(1, 0, 0, 0);
		ChannelPreset = EChannelPresets::None;
	}

	if (ImagePane.IsValid())
	{
		ImagePane->SetPreviewColorMask(ColorMask);
	}
}

void SCompElementPreviewDialog::SetChannelGreen()
{
	FLinearColor ColorMask(1, 1, 1, 0);
	if (ImagePane.IsValid())
	{
		ColorMask = ImagePane->GetPreviewColorMask();
	}

	if (ColorMask.R == 0 && ColorMask.G == 1 && ColorMask.B == 0 && ColorMask.A == 0)
	{
		ColorMask = FLinearColor(1, 1, 1, 0);
		ChannelPreset = EChannelPresets::RGB;
	}
	else
	{
		ColorMask = FLinearColor(0, 1, 0, 0);
		ChannelPreset = EChannelPresets::None;
	}

	if (ImagePane.IsValid())
	{
		ImagePane->SetPreviewColorMask(ColorMask);
	}
}

void SCompElementPreviewDialog::SetChannelBlue()
{
	FLinearColor ColorMask(1, 1, 1, 0);
	if (ImagePane.IsValid())
	{
		ColorMask = ImagePane->GetPreviewColorMask();
	}

	if (ColorMask.R == 0 && ColorMask.G == 0 && ColorMask.B == 1 && ColorMask.A == 0)
	{
		ColorMask = FLinearColor(1, 1, 1, 0);
		ChannelPreset = EChannelPresets::RGB;
	}
	else
	{
		ColorMask = FLinearColor(0, 0, 1, 0);
		ChannelPreset = EChannelPresets::None;
	}

	if (ImagePane.IsValid())
	{
		ImagePane->SetPreviewColorMask(ColorMask);
	}
}

void SCompElementPreviewDialog::SetChannelAlpha()
{
	FLinearColor ColorMask(1, 1, 1, 0);
	if (ImagePane.IsValid())
	{
		ColorMask = ImagePane->GetPreviewColorMask();
	}

	if (ColorMask.R == 0 && ColorMask.G == 0 && ColorMask.B == 0 && ColorMask.A == 1)
	{
		ColorMask = FLinearColor(1, 1, 1, 0);
		ChannelPreset = EChannelPresets::RGB;
	}
	else
	{
		ColorMask = FLinearColor(0, 0, 0, 1);
		ChannelPreset = EChannelPresets::A;
	}

	if (ImagePane.IsValid())
	{
		ImagePane->SetPreviewColorMask(ColorMask);
	}
}

void SCompElementPreviewDialog::SyncColorMaskPreset(const FLinearColor& ColorMask)
{
	if (ColorMask.R == 1 && ColorMask.G == 1 && ColorMask.B == 1)
	{
		if (ColorMask.A == 1)
		{
			ChannelPreset = EChannelPresets::RGBA;
		}
		else
		{
			ChannelPreset = EChannelPresets::RGB;
		}
	}
	else if (ColorMask.R == 0 && ColorMask.G == 0 && ColorMask.B == 0 && ColorMask.A == 1)
	{
		ChannelPreset = EChannelPresets::A;
	}
	else
	{
		ChannelPreset = EChannelPresets::None;
	}
}

#undef LOCTEXT_NAMESPACE
