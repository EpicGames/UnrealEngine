// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "EditorSupport/CompEditorImagePreviewInterface.h"
#include "EditorSupport/WeakUInterfacePtr.h"

class FUICommandList;

class SWindow;
class ICompEditorImagePreviewInterface;
class UMaterialInstanceDynamic;
class FReferenceCollector;
class SHorizontalBox;
class ICompElementManager;
class SCompElementPreviewPane;
struct FSlateBrush;
enum class EChannelPresets : uint8;
enum class EActiveTimerReturnType : uint8;
enum class ESlateDrawEffect : uint8;

class SCompElementPreviewDialog : public SCompoundWidget
{
public:
	static TSharedRef<SWindow> OpenPreviewWindow(TWeakUIntrfacePtr<ICompEditorImagePreviewInterface> PreviewTarget, TSharedPtr<SWidget> ParentWidget = nullptr, const FText& WindowTitle = FText());

public:
	 SCompElementPreviewDialog();
	~SCompElementPreviewDialog();

	SLATE_BEGIN_ARGS(SCompElementPreviewDialog)	{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(TWeakUIntrfacePtr<ICompEditorImagePreviewInterface>, PreviewTarget)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

public:
	//~ Begin SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	//~ End SWidget interface

protected:
	virtual TSharedRef<SWidget> GeneratePreviewContent();
	virtual void ExtendMenuOverlay(TSharedRef<SHorizontalBox> MenuBar) {}
	virtual TSharedRef<SHorizontalBox> GenerateHoveredColorOverlay();
	void BindCommands();
	
protected:
	TWeakUIntrfacePtr<ICompEditorImagePreviewInterface> PreviewTarget;
	/** A pointer to the window that is asking the user to select a parent class */
	TWeakPtr<SWindow> WeakParentWindow;

	static bool CanAlwaysExecute()
	{
		return true;
	}

	TSharedPtr<SCompElementPreviewPane> ImagePane;

private:
	void OnCycleChannelPresets();
	void SetChannelRed();
	void SetChannelGreen();
	void SetChannelBlue();
	void SetChannelAlpha();
	void SyncColorMaskPreset(const FLinearColor& ColorMask);

	FLinearColor ColorUnderMouse;
	TSharedPtr<FUICommandList> CommandList;
	EChannelPresets ChannelPreset;
};

