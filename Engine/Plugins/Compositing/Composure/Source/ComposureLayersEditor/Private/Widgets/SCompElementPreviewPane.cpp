// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCompElementPreviewPane.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Input/SButton.h"
#include "Materials/MaterialInterface.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateMaterialBrush.h"
#include "Modules/ModuleManager.h"
#include "CompElementEditorModule.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/Texture.h"
#include "UObject/Package.h"
#include "Widgets/SCompPreviewImage.h"
#include "ICompElementManager.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"

static TAutoConsoleVariable<int32> CVarForceApplyGamma(
	TEXT("r.Composure.CompositingElements.Editor.ForceApplyGammaToPreview"),
	0,
	TEXT("By default we don't do gamma correction if the user has set up a preview transform on the compositing tree. \n")
	TEXT("If you'd like to use a preview transform, and apply gamma on top of that, then enable this settong."));


#define LOCTEXT_NAMESPACE "CompElementPreviewPane"

SCompElementPreviewPane::SCompElementPreviewPane()
	: PreviewMaterialPath(TEXT("/Composure/Materials/Debuging/EditorPreviewMat"))
	, CommandList(MakeShareable(new FUICommandList))
	, ColorMask(FLinearColor(1, 1, 1, 0))
{}

SCompElementPreviewPane::~SCompElementPreviewPane()
{
	if (PreviewTarget.IsValid())
	{
		PreviewTarget->OnEndPreview();
	}

	if (ImageWidget.IsValid())
	{
		ImageWidget->SetImage(nullptr);
	}

	if (PreviewBrush.IsValid())
	{
		PreviewBrush->SetResourceObject(nullptr);
	}
}

void SCompElementPreviewPane::Construct(const FArguments& InArgs)
{
	const FSlateBrush* ImageBrush = FAppStyle::GetBrush("EditorViewportToolBar.MenuDropdown");
	const float MenuIconSize = 16.0f;
	const FMargin ToolbarSlotPadding(2.0f, 2.0f);

	UMaterialInterface* PreviewMaterial = Cast<UMaterialInterface>(PreviewMaterialPath.TryLoad());
	if (PreviewMaterial)
	{
		PreviewMID = UMaterialInstanceDynamic::Create(PreviewMaterial, GetTransientPackage());
		PreviewBrush = MakeShareable(new FSlateMaterialBrush(*PreviewMID, FVector2D(1920.f, 1080.f)));
	}

	PreviewTarget = InArgs._PreviewTarget;
	if (PreviewTarget.IsValid())
	{
		PreviewTarget->OnBeginPreview();

		UTexture* DisplayImage = PreviewTarget->GetEditorPreviewImage();
		SetDisplayImage(DisplayImage);
	}

	OnColorMaskChanged = InArgs._OnColorMaskChanged;
	OnRedraw = InArgs._OnRedraw;

	TSharedPtr<SHorizontalBox> MenuBarPtr;
	TSharedPtr<SOverlay> OverlayPtr;

	this->ChildSlot
	[
		SAssignNew(OverlayPtr, SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SAssignNew(ImageWidget, SCompPreviewImage)
					.Image(this, &SCompElementPreviewPane::GetDisplayImage)
					.DrawEffects(this, &SCompElementPreviewPane::GetDrawEffects)
			]
		]
		+ SOverlay::Slot()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Left)
		[
			SAssignNew(MenuBarPtr, SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(MenuAnchor, SMenuAnchor)
				.Placement(MenuPlacement_BelowAnchor)
				.OnGetMenuContent(this, &SCompElementPreviewPane::GenerateMenu)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(ToolbarSlotPadding)
						[
							SNew(SButton)
							// Allows users to drag with the mouse to select options after opening the menu */
							.ClickMethod(EButtonClickMethod::MouseDown)
							.ContentPadding(FMargin(5.0f, 2.0f))
							.VAlign(VAlign_Center)
							.ButtonStyle(FAppStyle::Get(), "EditorViewportToolBar.MenuButton")
							.OnClicked(this, &SCompElementPreviewPane::OnMenuClicked)
							[
								SNew(SBox)
								.HeightOverride(MenuIconSize)
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								[
									SNew(SImage)
									.Image(ImageBrush)
									.ColorAndOpacity(FSlateColor::UseForeground())
								]
							]
						]
					]
				]
			]
		]
	];

	InArgs._MenuOverlayExtender.ExecuteIfBound(MenuBarPtr.ToSharedRef());
	InArgs._OverlayExtender.ExecuteIfBound(OverlayPtr.ToSharedRef());


	ICompElementEditorModule& CompEditorModule = FModuleManager::GetModuleChecked<ICompElementEditorModule>(TEXT("ComposureLayersEditor"));
	CompElementManager = CompEditorModule.GetCompElementManager();
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SCompElementPreviewPane::RefreshRenderWindow));
}

void SCompElementPreviewPane::SetDisplayImage(UTexture* NewDisplayImage)
{
	if (PreviewMID != nullptr)
	{
		static const FName PreviewImgParamName(TEXT("PreviewImage"));
		if (NewDisplayImage == nullptr)
		{
			PreviewMID->GetTextureParameterDefaultValue(PreviewImgParamName, NewDisplayImage);
		}
		PreviewMID->SetTextureParameterValue(PreviewImgParamName, NewDisplayImage);

		PreviewMID->GetVectorParameterValue(TEXT("ChannelMask"), ColorMask);
	}

	if (PreviewBrush.IsValid())
	{
		FVector2D ImageSize(1920.f, 1080.f);
		if (NewDisplayImage)
		{
			ImageSize.X = NewDisplayImage->GetSurfaceWidth();
			ImageSize.Y = NewDisplayImage->GetSurfaceHeight();
		}
		PreviewBrush->ImageSize = ImageSize;
	}
}

void SCompElementPreviewPane::SetPreviewColorMask(const FLinearColor& NewColorMask)
{
	ColorMask = NewColorMask;
	ApplyColorMaskChange();
}

void SCompElementPreviewPane::SetOnRedraw(FSimpleDelegate InOnRedraw)
{
	OnRedraw = InOnRedraw;
}

void SCompElementPreviewPane::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (PreviewMID)
	{
		Collector.AddReferencedObject(PreviewMID);
	}
}

const FSlateBrush* SCompElementPreviewPane::GetDisplayImage() const
{
	FSlateBrush* OutBrush = nullptr;
	if (PreviewBrush.IsValid())
	{
		OutBrush = PreviewBrush.Get();
	}
	return OutBrush;
}

ESlateDrawEffect SCompElementPreviewPane::GetDrawEffects() const
{
	if (PreviewTarget.IsValid() && !CVarForceApplyGamma.GetValueOnGameThread())
	{
		return PreviewTarget->UseImplicitGammaForPreview() ? ESlateDrawEffect::None : ESlateDrawEffect::NoGamma;
	}
	return ESlateDrawEffect::None;
}

EActiveTimerReturnType SCompElementPreviewPane::RefreshRenderWindow(double /*InCurrentTime*/, float /*InDeltaTime*/)
{
	bool bNeedsRedraw = false;
	if (PreviewTarget.IsValid())
	{
		UTexture* DisplayImage = PreviewTarget->GetEditorPreviewImage();
		SetDisplayImage(DisplayImage);

		bNeedsRedraw = true;
	}

	if (CompElementManager.IsValid())
	{
		CompElementManager->RequestRedraw();
	}

	OnRedraw.ExecuteIfBound();

	return bNeedsRedraw ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
}

FReply SCompElementPreviewPane::OnMenuClicked()
{
	// If the menu button is clicked toggle the state of the menu anchor which will open or close the menu
	if (MenuAnchor->ShouldOpenDueToClick())
	{
		MenuAnchor->SetIsOpen(true);
	}
	else
	{
		MenuAnchor->SetIsOpen(false);
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SCompElementPreviewPane::GenerateMenu() const
{
	const bool bInShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder OptionsMenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList);
	{
		OptionsMenuBuilder.BeginSection("ComposurePreviewColorMask", LOCTEXT("ColorMaskSection", "Color Channels"));
		OptionsMenuBuilder.AddMenuEntry(
			LOCTEXT("RedChannel", "Red"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(const_cast<SCompElementPreviewPane*>(this), &SCompElementPreviewPane::RedChannelToggled),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(const_cast<SCompElementPreviewPane*>(this), &SCompElementPreviewPane::GetRedChannel)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		OptionsMenuBuilder.AddMenuEntry(
			LOCTEXT("GreenChannel", "Green"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(const_cast<SCompElementPreviewPane*>(this), &SCompElementPreviewPane::GreenChannelToggled),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(const_cast<SCompElementPreviewPane*>(this), &SCompElementPreviewPane::GetGreenChannel)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		OptionsMenuBuilder.AddMenuEntry(
			LOCTEXT("BluedChannel", "Blue"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(const_cast<SCompElementPreviewPane*>(this), &SCompElementPreviewPane::BlueChannelToggled),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(const_cast<SCompElementPreviewPane*>(this), &SCompElementPreviewPane::GetBlueChannel)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		OptionsMenuBuilder.AddMenuEntry(
			LOCTEXT("AlphaChannel", "Alpha"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(const_cast<SCompElementPreviewPane*>(this), &SCompElementPreviewPane::AlphaChannelToggled),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(const_cast<SCompElementPreviewPane*>(this), &SCompElementPreviewPane::GetAlphaChannel)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		OptionsMenuBuilder.EndSection();
	}
	return OptionsMenuBuilder.MakeWidget();
}

void SCompElementPreviewPane::RedChannelToggled()
{
	ColorMask.R = (ColorMask.R == 0 ? 1 : 0);
	ApplyColorMaskChange();
}

ECheckBoxState SCompElementPreviewPane::GetRedChannel()
{
	return (ColorMask.R == 0 ? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
}

void SCompElementPreviewPane::GreenChannelToggled()
{
	ColorMask.G = (ColorMask.G == 0 ? 1 : 0);
	ApplyColorMaskChange();
}

ECheckBoxState SCompElementPreviewPane::GetGreenChannel()
{
	return (ColorMask.G == 0 ? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
}

void SCompElementPreviewPane::BlueChannelToggled()
{
	ColorMask.B = (ColorMask.B == 0 ? 1 : 0);
	ApplyColorMaskChange();
}

ECheckBoxState SCompElementPreviewPane::GetBlueChannel()
{
	return (ColorMask.B == 0 ? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
}

void SCompElementPreviewPane::AlphaChannelToggled()
{
	ColorMask.A = (ColorMask.A == 0 ? 1 : 0);
	ApplyColorMaskChange();
}

ECheckBoxState SCompElementPreviewPane::GetAlphaChannel()
{
	return (ColorMask.A == 0 ? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
}

void SCompElementPreviewPane::ApplyColorMaskChange()
{
	if (PreviewMID)
	{
		PreviewMID->SetVectorParameterValue(TEXT("ChannelMask"), ColorMask);
	}

	if (OnColorMaskChanged.IsBound())
	{
		OnColorMaskChanged.Execute(ColorMask);
	}
}

#undef LOCTEXT_NAMESPACE
