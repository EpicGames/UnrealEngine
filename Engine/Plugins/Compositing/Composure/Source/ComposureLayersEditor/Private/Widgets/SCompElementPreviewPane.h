// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSupport/CompEditorImagePreviewInterface.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"
#include "EditorSupport/WeakUInterfacePtr.h"

class ICompElementManager;
class UTexture;
class SMenuAnchor;
class SImage;
class FUICommandList;
class UMaterialInstanceDynamic;
struct FSlateBrush;
enum class ECheckBoxState : uint8;
enum class EActiveTimerReturnType : uint8;
enum class ESlateDrawEffect : uint8;
class SHorizontalBox;
class SOverlay;

class SCompElementPreviewPane : public SCompoundWidget, public FGCObject
{
public:
	 SCompElementPreviewPane();
	~SCompElementPreviewPane();

	DECLARE_DELEGATE_OneParam(FOverlayExtender, TSharedRef<SOverlay> /*OverlayWidget*/);
	DECLARE_DELEGATE_OneParam(FMenuBarOverlayExtender, TSharedRef<SHorizontalBox> /*MenuBarWidget*/);
	DECLARE_DELEGATE_OneParam(FOnColorMaskChange, const FLinearColor& /*NewColor*/);

	SLATE_BEGIN_ARGS(SCompElementPreviewPane) {}
		SLATE_ARGUMENT(TWeakUIntrfacePtr<ICompEditorImagePreviewInterface>, PreviewTarget)
		SLATE_EVENT(FOverlayExtender, OverlayExtender)
		SLATE_EVENT(FMenuBarOverlayExtender, MenuOverlayExtender)
		SLATE_EVENT(FOnColorMaskChange, OnColorMaskChanged)
		SLATE_EVENT(FSimpleDelegate, OnRedraw)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetDisplayImage(UTexture* NewDisplayImage);
	void SetPreviewColorMask(const FLinearColor& NewColorMask);
	const FLinearColor& GetPreviewColorMask() const { return ColorMask; }
	
	void SetOnRedraw(FSimpleDelegate OnRedraw);

public:
	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SCompElementPreviewPane");
	}

private:
	virtual const FSlateBrush* GetDisplayImage() const;
	ESlateDrawEffect GetDrawEffects() const;

	FReply OnMenuClicked();
	TSharedRef<SWidget> GenerateMenu() const;

	void RedChannelToggled();
	ECheckBoxState GetRedChannel();
	void GreenChannelToggled();
	ECheckBoxState GetGreenChannel();
	void BlueChannelToggled();
	ECheckBoxState GetBlueChannel();
	void AlphaChannelToggled();
	ECheckBoxState GetAlphaChannel();

	void ApplyColorMaskChange();

	EActiveTimerReturnType RefreshRenderWindow(double InCurrentTime, float InDeltaTime);

private:
	TSharedPtr<FSlateBrush> PreviewBrush;
	FSoftObjectPath PreviewMaterialPath;
	TObjectPtr<UMaterialInstanceDynamic> PreviewMID;
	TSharedPtr<SImage> ImageWidget;

	TSharedPtr<SMenuAnchor> MenuAnchor;
	TSharedPtr<FUICommandList> CommandList;
	FLinearColor ColorMask;

	TSharedPtr<ICompElementManager> CompElementManager;

	TWeakUIntrfacePtr<ICompEditorImagePreviewInterface> PreviewTarget;
	FOnColorMaskChange OnColorMaskChanged;
	FSimpleDelegate OnRedraw;
};
