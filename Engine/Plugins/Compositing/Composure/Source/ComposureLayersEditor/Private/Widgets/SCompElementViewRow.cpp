// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCompElementViewRow.h"
#include "CompElementViewModel.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Input/SComboButton.h"
#include "Layout/WidgetPath.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/SCompElementPreviewDialog.h"
#include "CompositingElement.h"
#include "ComposureEditorStyle.h"
#include "CompElementDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Editor.h" // for GEditor
#include "CompElementEditorCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "CompElementsView"

/* SContextMenuButton
 *****************************************************************************/

class SContextMenuButton : public SButton
{
public:
	DECLARE_DELEGATE_RetVal(TSharedPtr<SWidget>, FOnContextMenuOpening)
	FOnContextMenuOpening ConstructContextMenu;

public:
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			if (ConstructContextMenu.IsBound())
			{
				TSharedPtr<SWidget> MenuContents = ConstructContextMenu.Execute();
				if (MenuContents.IsValid())
				{
					FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
					FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContents.ToSharedRef(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

					return FReply::Handled();
				}
			}
		}
		return SButton::OnMouseButtonUp(MyGeometry, MouseEvent);
	}
};

/* SCompElementViewRow
 *****************************************************************************/

void SCompElementViewRow::Construct(const FArguments& InArgs, TSharedRef< FCompElementViewModel > InViewModel, TSharedRef< STableViewBase > InOwnerTableView)
{
	ViewModel = InViewModel;

	HighlightText = InArgs._HighlightText;

	SMultiColumnTableRow< TSharedPtr< FCompElementViewModel > >::Construct(FSuperRowType::FArguments().OnDragDetected(InArgs._OnDragDetected), InOwnerTableView);
}

SCompElementViewRow::~SCompElementViewRow()
{
	ViewModel->OnPreviewRequest().Remove(PreviewRequestDelegateHandle);
	ViewModel->OnRenamedRequest().Remove(EnterEditingModeDelegateHandle);
}

TSharedRef<SWidget> SCompElementViewRow::GenerateWidgetForColumn(const FName& ColumnID)
{
	TSharedPtr<SWidget> TableRowContent;

	if (ColumnID == CompElementsView::ColumnID_ElementLabel)
	{
		TableRowContent =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
				.VAlign(VAlign_Center)
			.Padding(FDefaultTreeItemMetrics::IconPadding())
			[
				SNew(SBox)
				.WidthOverride(FDefaultTreeItemMetrics::IconSize())
				.HeightOverride(FDefaultTreeItemMetrics::IconSize())
				[
					GetIcon()
				]
			]

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
					.Font(FAppStyle::GetFontStyle("LayersView.LayerNameFont"))
					.Text(ViewModel.Get(), &FCompElementViewModel::GetNameAsText)
					.ColorAndOpacity(this, &SCompElementViewRow::GetColorAndOpacity)
					.HighlightText(HighlightText)
					.MaximumLength(NAME_SIZE-1)
					.OnVerifyTextChanged(this, &SCompElementViewRow::OnRenameElementTextChanged)
					.OnTextCommitted(this, &SCompElementViewRow::OnRenameElementTextCommitted)
					.IsSelected(this, &SCompElementViewRow::IsSelectedExclusively)
					.IsEnabled(ViewModel.Get(), &FCompElementViewModel::IsEditable)
			]
		;

		EnterEditingModeDelegateHandle = ViewModel->OnRenamedRequest().AddSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
		PreviewRequestDelegateHandle   = ViewModel->OnPreviewRequest().AddRaw(this, &SCompElementViewRow::OnPreviewRequested);
	}
	else if (ColumnID == CompElementsView::ColumnID_Visibility)
	{
		TableRowContent =
			SAssignNew(VisibilityButton, SButton)
			.ContentPadding(0)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.OnClicked(this, &SCompElementViewRow::OnToggleVisibility)
			.ToolTipText(LOCTEXT("RenderingButtonToolTip", "Toggle Element Rendering"))
			.IsEnabled(this, &SCompElementViewRow::VisibilityToggleEnabled)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Content()
			[
				SNew(SImage)
				.Image(this, &SCompElementViewRow::GetVisibilityBrushForElement)
			]
		;
	}
	else if (ColumnID == CompElementsView::ColumnID_Alpha)
	{
		struct FPercentTypeInterface : public TDefaultNumericTypeInterface<float>
		{
			virtual FString ToString(const float& Value) const override
			{
				static const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
					.SetUseGrouping(false)
					.SetMinimumFractionalDigits(0)
					.SetMaximumFractionalDigits(2);
				FString NumberString = FastDecimalFormat::NumberToString(Value * 100.f, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
				return NumberString + TEXT('%');
			}

			virtual TOptional<float> FromString(const FString& InString, const float& ExistingValue)
			{
				TOptional<float> ParsedValue = TDefaultNumericTypeInterface<float>::FromString(InString, ExistingValue);
				if (ParsedValue.IsSet())
				{
					return ParsedValue.GetValue() / 100.f;
				}
				return ParsedValue;
			}
		};

		TableRowContent = SNew(SBox)
			.MinDesiredWidth(114.f)
			.MaxDesiredWidth(114.f)
			[
				SNew(SComboButton)
					.ContentPadding(FMargin(0, 0, 5, 0))
					.ToolTipText(LOCTEXT("OpacityComboTooltip", "Opacity"))
					.IsEnabled(this, &SCompElementViewRow::IsAlphaWidgetEnabled)
				.ButtonContent()
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("NoBorder"))
						.Padding(FMargin(0, 0, 5, 0))
					[
						SNew(SNumericEntryBox<float>)
							.Value(this, &SCompElementViewRow::GetAlphaValueOptional)
							.OnValueChanged(this, &SCompElementViewRow::OnSetAlphaValue, /*bFromSlider =*/false)
							.OnValueCommitted(this, &SCompElementViewRow::OnCommitAlphaValue)
							.TypeInterface(MakeShareable(new FPercentTypeInterface))
							.MinValue(0.0f)
							.MaxValue(1.0f)
							.Font(FAppStyle::GetFontStyle("LayersView.LayerNameFont"))
					]
				]
				.MenuContent()
				[
					SNew(SSlider)
						.Value(this, &SCompElementViewRow::GetAlphaValue)
						.OnValueChanged(this, &SCompElementViewRow::OnSetAlphaValue, /*bFromSlider =*/true)
						.SliderBarColor(FLinearColor(0.48f, 0.48f, 0.48f))
						.Style(FComposureEditorStyle::Get(), "ComposureTree.AlphaScrubber")
						//.MouseUsesStep(true)
						.StepSize(0.01)
						.OnMouseCaptureEnd(this, &SCompElementViewRow::OnAlphaSliderMouseEnd)
				]
			];
	}
	else if (ColumnID == CompElementsView::ColumnID_MediaCapture)
	{
		TSharedPtr<SContextMenuButton> MediaCaptureToggle;

		TableRowContent =
			SAssignNew(MediaCaptureToggle, SContextMenuButton)
			.ContentPadding(0)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ToolTipText(LOCTEXT("MediaCaptureToggleTooltip", "Turn Media Capture On/Off"))
			.OnClicked(this, &SCompElementViewRow::OnToggleMediaCapture)
			.IsEnabled(this, &SCompElementViewRow::IsMediaCaptureToggleEnabled)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Content()
			[
				SNew(SBox)
				.MinDesiredWidth(16.f)
				[
					SNew(SImage).Image(this, &SCompElementViewRow::GetMediaCaptureStatusBrush)
				]
				
			]
		;

		MediaCaptureToggle->ConstructContextMenu.BindSP(this, &SCompElementViewRow::CreateMediaCaptureToggleContextMenu);
	}
	else if (ColumnID == CompElementsView::ColumnID_FreezeFrame)
	{
		TableRowContent =
			SAssignNew(FreezeFrameButton, SButton)
			.ContentPadding(0)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ToolTipText(LOCTEXT("FreezeToggleTooltip", "Toggle Freeze Framing"))
			.OnClicked(this, &SCompElementViewRow::OnToggleFreezeFrame)
			.IsEnabled(this, &SCompElementViewRow::IsFreezeFrameToggleEnabled)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Content()
			[
				SNew(SImage)
				.Image(this, &SCompElementViewRow::GetFreezeFrameBrushForElement)
			]
		;
	}
	else
	{
		checkf(false, TEXT("Unknown ColumnID provided to SCompElementsView"));
	}

	return TableRowContent.ToSharedRef();
}

void SCompElementViewRow::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr< FCompElementDragDropOp > DragActorOp = DragDropEvent.GetOperationAs< FCompElementDragDropOp >();
	if (DragActorOp.IsValid())
	{
		DragActorOp->ResetToDefaultToolTip();
	}
}

FReply SCompElementViewRow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr< FCompElementDragDropOp > DragActorOp = DragDropEvent.GetOperationAs< FCompElementDragDropOp >();
	if (!DragActorOp.IsValid())
	{
		return FReply::Unhandled();
	}
	return FReply::Handled();
}

FReply SCompElementViewRow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr< FCompElementDragDropOp > DragActorOp = DragDropEvent.GetOperationAs< FCompElementDragDropOp >();
	if (!DragActorOp.IsValid())
	{
		return FReply::Unhandled();
	}

	ViewModel->AttachCompElements(DragActorOp->Elements);

	return FReply::Handled();
}

bool SCompElementViewRow::OnRenameElementTextChanged(const FText& NewText, FText& OutErrorMessage)
{
	FString OutMessage;
	if (!ViewModel->CanRenameTo(*NewText.ToString(), OutMessage))
	{
		OutErrorMessage = FText::FromString(OutMessage);
		return false;
	}

	return true;
}

void SCompElementViewRow::OnRenameElementTextCommitted(const FText& InText, ETextCommit::Type eInCommitType)
{
	if (!InText.IsEmpty())
	{
		ViewModel->RenameTo(*InText.ToString());
	}
}

FSlateColor SCompElementViewRow::GetColorAndOpacity() const
{
	if (!FSlateApplication::Get().IsDragDropping())
	{
		return FSlateColor::UseForeground();
	}

	bool bCanAcceptDrop = false;
	TSharedPtr<FDragDropOperation> DragDropOp = FSlateApplication::Get().GetDragDroppingContent();
	if (DragDropOp.IsValid() && DragDropOp->IsOfType<FCompElementDragDropOp>())
	{
		bCanAcceptDrop = true;
	}
	else if (DragDropOp.IsValid() && DragDropOp->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> DragDropAssetOp = StaticCastSharedPtr<FAssetDragDropOp>(DragDropOp);

		FText Message;
		//TODO: Check if there is one asset, it's a BP, and it's base is ComposureCompShotElement
		bCanAcceptDrop = false;
	}

	return (bCanAcceptDrop) ? FSlateColor::UseForeground() : FLinearColor(0.30f, 0.30f, 0.30f);
}

TSharedRef< SWidget > SCompElementViewRow::GetIcon()
{
	TWeakObjectPtr<ACompositingElement> ElementPtr = ViewModel->GetDataSource();

	return SNew(SImage)
		.Image(FClassIconFinder::FindIconForActor(ElementPtr));

}

void SCompElementViewRow::OnPreviewRequested()
{
	if (PreviewWindow.IsValid())
	{
		PreviewWindow.Pin()->RequestDestroyWindow();
		PreviewWindow.Reset();
		// @TODO: should probably focus this instead of destroying and recreating
	}

	TWeakObjectPtr<ACompositingElement> ElementPtr = ViewModel->GetDataSource();
	if (ElementPtr.IsValid())
	{
		const FText WindowTitle = FText::Format( LOCTEXT("PreviewWindowTitle", "Preview: {0}"), FText::FromName(ElementPtr->GetCompElementName()) );

		TWeakUIntrfacePtr<ICompEditorImagePreviewInterface> PreviewTarget(ElementPtr.Get());
		PreviewWindow = SCompElementPreviewDialog::OpenPreviewWindow(PreviewTarget, SharedThis(this), WindowTitle);
	}
}

FReply SCompElementViewRow::OnToggleVisibility()
{
	ViewModel->ToggleRendering();
	if (GEditor)
	{
		GEditor->RedrawAllViewports(/*bInvalidateHitProxies =*/false);
	}

	return FReply::Handled();
}

const FSlateBrush* SCompElementViewRow::GetVisibilityBrushForElement() const
{
	if (ViewModel->IsSetToRender() && !ViewModel->IsRenderingExternallyDisabled())
	{
		return IsHovered() ? FAppStyle::GetBrush("Level.VisibleHighlightIcon16x") :
			FAppStyle::GetBrush("Level.VisibleIcon16x");
	}
	else
	{
		return IsHovered() ? FAppStyle::GetBrush("Level.NotVisibleHighlightIcon16x") :
			FAppStyle::GetBrush("Level.NotVisibleIcon16x");
	}
}

bool SCompElementViewRow::VisibilityToggleEnabled() const
{
	return !ViewModel->IsRenderingExternallyDisabled();
}

const FSlateBrush* SCompElementViewRow::GetFreezeFrameBrushForElement() const
{
	if (ViewModel->IsFrameFrozen())
	{
		return IsHovered() ? FComposureEditorStyle::Get().GetBrush("ComposureTree.FrameFrozenHighlightIcon16x") :
			FComposureEditorStyle::Get().GetBrush("ComposureTree.FrameFrozenIcon16x");
	}
	else
	{
		return IsHovered() ? FComposureEditorStyle::Get().GetBrush("ComposureTree.NoFreezeFrameHighlightIcon16x") :
			FComposureEditorStyle::Get().GetBrush("ComposureTree.NoFreezeFrameIcon16x");
	}
}

FReply SCompElementViewRow::OnToggleFreezeFrame()
{
	ViewModel->ToggleFreezeFrame();
	if (!ViewModel->IsFrameFrozen() && GEditor)
	{
		GEditor->RedrawAllViewports(/*bInvalidateHitProxies =*/false);
	}

	return FReply::Handled();
}

bool SCompElementViewRow::IsFreezeFrameToggleEnabled() const
{
	return ViewModel->IsFreezeFramingPermitted();
}

const FSlateBrush* SCompElementViewRow::GetMediaCaptureStatusBrush() const
{
	bool bIsOutputActive  = false;
	bool bHasMediaCapture = ViewModel->HasMediaCaptureSetup(bIsOutputActive);
	
	return bIsOutputActive ? FComposureEditorStyle::Get().GetBrush("ComposureTree.MediaCaptureOn16x") :
		bHasMediaCapture ? FComposureEditorStyle::Get().GetBrush("ComposureTree.MediaCaptureOff16x") : FComposureEditorStyle::Get().GetBrush("ComposureTree.NoMediaCapture16x");
}

FReply SCompElementViewRow::OnToggleMediaCapture()
{
	ViewModel->ToggleMediaCapture();
	return FReply::Handled();
}

bool SCompElementViewRow::IsMediaCaptureToggleEnabled() const
{
	return ViewModel->IsSetToRender();
}

TSharedPtr<SWidget> SCompElementViewRow::CreateMediaCaptureToggleContextMenu()
{
	FMenuBuilder MenuBuilder(/*bShouldCloseWindowAfterMenuSelection =*/true, ViewModel->GetCommandList());
	const FCompElementEditorCommands& Commands = FCompElementEditorCommands::Get();

	MenuBuilder.BeginSection("MediaOutputSection", LOCTEXT("MediaOutputHeader", "Media Capture Output"));
	{
		MenuBuilder.AddMenuEntry(Commands.ResetMediaOutput);
		MenuBuilder.AddMenuEntry(Commands.RemoveMediaOutput);
	}
	MenuBuilder.EndSection();
	

	return MenuBuilder.MakeWidget();
}

TOptional<float> SCompElementViewRow::GetAlphaValueOptional() const
{
	return GetAlphaValue();
}

float SCompElementViewRow::GetAlphaValue() const
{
	return ViewModel->GetElementOpacity();
}

void SCompElementViewRow::OnSetAlphaValue(float NewValue, bool bFromSlider)
{
	if (bFromSlider)
	{
		if (!bSettingAlphaInteractively)
		{
			GEditor->BeginTransaction(LOCTEXT("SetElementOpacity", "Set Element Opacity"));
			bSettingAlphaInteractively = true;
		}

		NewValue = FMath::RoundToFloat(NewValue * 100.f) / 100.f;
	}

	ViewModel->SetElementOpacity(FMath::Clamp(NewValue, 0.f, 1.f), /*bInteractive =*/bFromSlider);
}

void SCompElementViewRow::OnCommitAlphaValue(float NewValue, ETextCommit::Type CommitType)
{
	OnSetAlphaValue(NewValue, /*bFromSlider =*/false);
}

void SCompElementViewRow::OnAlphaSliderMouseEnd()
{
	if (bSettingAlphaInteractively)
	{
		// set the value non-interactively, so we log a transaction
		ViewModel->SetElementOpacity(GetAlphaValue());
		GEditor->EndTransaction();

		bSettingAlphaInteractively = false;
	}
}

bool SCompElementViewRow::IsAlphaWidgetEnabled() const
{
	return ViewModel->IsOpacitySettingEnabled();
}

#undef LOCTEXT_NAMESPACE
