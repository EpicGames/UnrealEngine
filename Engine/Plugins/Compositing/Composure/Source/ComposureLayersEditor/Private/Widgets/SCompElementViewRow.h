// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/STableRow.h"

class FCompElementViewModel;
class SInlineEditableTextBlock;
class STableViewBase;

class SButton;

namespace CompElementsView
{
	/** IDs for list columns */
	static const FName ColumnID_ElementLabel("Element");
	static const FName ColumnID_Visibility("Visibility");
	static const FName ColumnID_Alpha("Alpha");
	static const FName ColumnID_FreezeFrame("FreezeFrame");
	static const FName ColumnID_MediaCapture("MediaCapture");
}

/**
 * The widget that represents a row in the element view's list-view control. Generates widgets for each column on demand.
 */
class SCompElementViewRow : public SMultiColumnTableRow< TSharedPtr<FCompElementViewModel> >
{
public:
	SLATE_BEGIN_ARGS(SCompElementViewRow) {}
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
	SLATE_END_ARGS()

	/**
	 * Construct this widget. Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The element the row widget is supposed to represent
	 * @param	InOwnerTableView	The owner of the row widget
	 */
	void Construct(const FArguments& InArgs, TSharedRef<FCompElementViewModel> InViewModel, TSharedRef<STableViewBase> InOwnerTableView);

	~SCompElementViewRow();

protected:
	//~ Begin SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnID) override;
	//~ End SMultiColumnTableRow interface

	//~ Begin SWidget interface
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//~ End SWidget interface

	/** Callback when the SInlineEditableTextBlock is committed, to update the name of the element this row represents. */
	void OnRenameElementTextCommitted(const FText& InText, ETextCommit::Type eInCommitType);
	/** Callback when the SInlineEditableTextBlock is changed, to check for error conditions. */
	bool OnRenameElementTextChanged(const FText& NewText, FText& OutErrorMessage);	

private:
	/**
	 *	Returns the color and opacity for displaying the bound element's name.
	 *	The color and opacity changes depending on whether a drag/drop operation is occurring.
	 *
	 *	@return	The SlateColor to render the element's name in
	 */
	FSlateColor GetColorAndOpacity() const;

	/**
	 * Returns the row's leading icon (associated with the wrapped element type).
	 */
	TSharedRef<SWidget> GetIcon();

	/** 
	 * Opens a standalone preview window displaying the associated element.
	 */
	void OnPreviewRequested();

	/**
	 *	Called when the user clicks on the visibility icon for a element's row widget.
	 *
	 *	@return	A reply that indicated whether this event was handled
	 */
	FReply OnToggleVisibility();	

	/**
	 *	Called to get the SlateImageBrush representing the visibility state of
	 *	the element this row widget represents
	 *
	 *	@return	The SlateBrush representing the element's visibility state
	 */
	const FSlateBrush* GetVisibilityBrushForElement() const;

	/** 
	 * Reports if the visibility button should be enabled or not (i.e. if toggling it would have an effect). 
	 */
	bool VisibilityToggleEnabled() const;

	const FSlateBrush* GetFreezeFrameBrushForElement() const;
	FReply OnToggleFreezeFrame();
	bool IsFreezeFrameToggleEnabled() const;

	const FSlateBrush* GetMediaCaptureStatusBrush() const;
	FReply OnToggleMediaCapture();
	bool IsMediaCaptureToggleEnabled() const;
	TSharedPtr<SWidget> CreateMediaCaptureToggleContextMenu();

	TOptional<float> GetAlphaValueOptional() const;
	float GetAlphaValue() const;
	void OnSetAlphaValue(float NewValue, bool bFromSlider);
	void OnCommitAlphaValue(float NewValue, ETextCommit::Type CommitType);
	void OnAlphaSliderMouseEnd();
	bool IsAlphaWidgetEnabled() const;

private:
	/** Default metrics for outliner tree items */
	struct FDefaultTreeItemMetrics
	{
		static int32	IconSize()    { return 18; };
		static FMargin	IconPadding() { return FMargin(0.f, 0.f, 6.f, 0.f); };
	};

	/** The element associated with this row of data */
	TSharedPtr<FCompElementViewModel> ViewModel;

	/**	The visibility button for the element */
	TSharedPtr<SButton> VisibilityButton;

	/** The button widget for toggling the element's paused state */
	TSharedPtr<SButton> FreezeFrameButton;

	/** The string to highlight on any text contained in the row widget */
	TAttribute<FText> HighlightText;

	/** Widget for displaying and editing the element name */
	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

	/** Tracks whether the alpha slider is currently being dragged or not. */
	bool bSettingAlphaInteractively = false;

	/** Handle to the registered EnterEditingMode delegate */
	FDelegateHandle EnterEditingModeDelegateHandle;
	/** Handle ti the registered OnPreviewRequest delegate */
	FDelegateHandle PreviewRequestDelegateHandle;

	/** Weak pointer to the active preview window - spawned from OnPreviewRequested() */
	TWeakPtr<SWindow> PreviewWindow;
};

