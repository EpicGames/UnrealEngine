// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CompElementCollectionViewModel.h"
#include "CompElementViewModel.h"
#include "Widgets/SCompElementViewRow.h"
#include "CompElementDragDropOp.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Views/STreeView.h"
#include "CompositingElement.h"
#include "Framework/Views/TableViewMetadata.h"

enum class ECompElementEdActions;

#define LOCTEXT_NAMESPACE "CompElementsView"

typedef STreeView< TSharedPtr<FCompElementViewModel> > SCompElementTreeView;

/**
 * A slate widget that can be used to display a list of compositing elements and perform various element related actions.
 */
class SCompElementsView : public SCompoundWidget
{
public:
	typedef SListView< TSharedPtr<FCompElementViewModel> >::FOnGenerateRow FOnGenerateRow;

	SLATE_BEGIN_ARGS(SCompElementsView) {}
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(FOnContextMenuOpening, ConstructContextMenu)
		SLATE_EVENT(FOnGenerateRow, OnGenerateRow)
	SLATE_END_ARGS()

	SCompElementsView()
		: bUpdatingSelection(false)
	{}

	~SCompElementsView()
	{
		ViewModel->OnElementsChanged().RemoveAll(this);
		ViewModel->OnSelectionChanged().RemoveAll(this);
	}

	/**
	 * Construct this widget. Called by the SNew() Slate macro.
	 *
	 * @param	InArgs		Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel The UI logic not specific to slate
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FCompElementCollectionViewModel>& InViewModel)
	{
		ViewModel = InViewModel;

		HighlightText = InArgs._HighlightText;
		FOnGenerateRow OnGenerateRowDelegate = InArgs._OnGenerateRow;

		if (!OnGenerateRowDelegate.IsBound())
		{
			OnGenerateRowDelegate = FOnGenerateRow::CreateSP(this, &SCompElementsView::OnGenerateRowDefault);
		}

		TSharedRef< SHeaderRow > HeaderRowWidget =
			SNew( SHeaderRow )

			/** We don't want the normal header to be visible */
			.Visibility(EVisibility::Collapsed)

			/** Element visibility column */
			+SHeaderRow::Column(CompElementsView::ColumnID_Visibility)
				.DefaultLabel(LOCTEXT("Visibility", "Visibility"))
				.FixedWidth(40.0f)

			/** ElementName label column */
			+SHeaderRow::Column(CompElementsView::ColumnID_ElementLabel)
				.DefaultLabel(LOCTEXT("Column_ElementNameLabel", "Element"))
				.FillWidth(0.45f)

			+ SHeaderRow::Column(CompElementsView::ColumnID_Alpha)
				.HAlignCell(HAlign_Right)
				.HAlignHeader(HAlign_Right)
				.FixedWidth(114.f)
				.DefaultLabel(LOCTEXT("Column_AlphaNameLabel", "Alpha"))

			+ SHeaderRow::Column(CompElementsView::ColumnID_MediaCapture)
				.HAlignCell(HAlign_Right)
				.HAlignHeader(HAlign_Right)
				.FixedWidth(24.f)
				.DefaultLabel(LOCTEXT("Column_MediaCaptureNameLabel", "Media Capture"))

			+ SHeaderRow::Column(CompElementsView::ColumnID_FreezeFrame)
				.HAlignCell(HAlign_Right)
				.HAlignHeader(HAlign_Right)
				.FixedWidth(24.f)
				.DefaultLabel(LOCTEXT("Column_FreezeFrameNameLabel", "Freeze Frame"));

		ChildSlot
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
					.FillHeight(1.0f)
				[
					SAssignNew(TreeView, SCompElementTreeView)
						.SelectionMode(ESelectionMode::Multi)
						.TreeItemsSource(&ViewModel->GetRootCompElements())
						.OnGetChildren(ViewModel.Get(), &FCompElementCollectionViewModel::GetChildElements)
						.OnSelectionChanged(this, &SCompElementsView::OnSelectionChanged)
						.OnMouseButtonDoubleClick( this, &SCompElementsView::OnListViewMouseButtonDoubleClick )
						.OnGenerateRow(OnGenerateRowDelegate) 
						.OnContextMenuOpening(InArgs._ConstructContextMenu)
						.HeaderRow(HeaderRowWidget)
						.OnItemScrolledIntoView(this, &SCompElementsView::OnItemScrolledIntoView)
				]
			];

		ViewModel->OnSelectionChanged().AddSP(this, &SCompElementsView::UpdateSelection);
		ViewModel->OnElementsChanged().AddSP(this, &SCompElementsView::RequestRefresh);
	}

	/** Requests a rename on the selected element, first forcing the item to scroll into view */
	void RequestRenameOnSelectedElement()
	{
		if (TreeView->GetNumItemsSelected() == 1)
		{
			RequestedRenameElement = TreeView->GetSelectedItems()[0];
			TreeView->RequestScrollIntoView(TreeView->GetSelectedItems()[0]);
		}
	}

protected:
	//~ Begin SWidget interface
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		return ViewModel->GetCommandList()->ProcessCommandBindings(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();
	}
	//~ End SWidget interface

private:
	/** 
	 * Called by the STreeView to generate a table row for the specified item.
	 *
	 * @param  Item			The FCompElementViewModel to generate a TableRow widget for
	 * @param  OwnerTable	The TableViewBase which will own the generated TableRow widget
	 *
	 * @return The generated TableRow widget
	 */
	TSharedRef<ITableRow> OnGenerateRowDefault(const TSharedPtr<FCompElementViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(SCompElementViewRow, Item.ToSharedRef(), OwnerTable)
			.HighlightText(HighlightText)
			.OnDragDetected(this, &SCompElementsView::OnDragRow);
	}

	/**
	 * Kicks off a Refresh of the elements view.
	 *
	 * @param  Action				The action taken on one or more elements
	 * @param  ChangedElement		The element that changed
	 * @param  ChangedProperty		The property that changed
	 */
	void RequestRefresh(const ECompElementEdActions /*Action*/, const TWeakObjectPtr<ACompositingElement>& /*ChangedElement*/, const FName& /*ChangedProperty*/)
	{
		TreeView->RequestTreeRefresh();
	}

	/**
	 * Called whenever the view model's selection changes.
	 */
	void UpdateSelection()
	{
		if (bUpdatingSelection)
		{
			return;
		}

		bUpdatingSelection = true;
		const auto& SelectedElements = ViewModel->GetSelectedElements();
		TreeView->ClearSelection();

		for (auto ElementIter = SelectedElements.CreateConstIterator(); ElementIter; ++ElementIter)
		{
			TreeView->SetItemSelection(*ElementIter, true);
		}

		if (SelectedElements.Num() == 1)
		{
			TreeView->RequestScrollIntoView(SelectedElements[0]);
		}
		bUpdatingSelection = false;
	}

	/** 
	 * Called by the STreeView when the selection has changed.
	 *
	 * @param  Item			The element affected by the selection change
	 * @param  SelectInfo	Provides context on how the selection changed
	 */
	void OnSelectionChanged(const TSharedPtr< FCompElementViewModel > Item, ESelectInfo::Type SelectInfo)
	{
		if (bUpdatingSelection)
		{
			return;
		}
		bUpdatingSelection = true;

		TArray< TSharedPtr<FCompElementViewModel> > SelectedTreeItems = TreeView->GetSelectedItems();
		for (int32 ElementIndex = SelectedTreeItems.Num() - 1; ElementIndex >= 0; --ElementIndex)
		{
			const TSharedPtr<FCompElementViewModel>& Element = SelectedTreeItems[ElementIndex];

			TSharedPtr<FCompElementViewModel> SelectionProxy = ViewModel->GetSelectionProxy(Element);
			if (Element != SelectionProxy)
			{
				TreeView->SetItemSelection(Element, false, ESelectInfo::Direct);
				SelectedTreeItems.RemoveAtSwap(ElementIndex);

				if (SelectionProxy.IsValid() && !SelectedTreeItems.Contains(SelectionProxy))
				{
					TreeView->SetItemSelection(SelectionProxy, true, ESelectInfo::Direct);
					SelectedTreeItems.Add(SelectionProxy);
				}
			}
		}

		ViewModel->SetSelectedElements(SelectedTreeItems);
		bUpdatingSelection = false;
	}

	/** 
	 * Called by the STreeView when the user double-clicks on an item.
	 *
	 * @param Item	The element that was double clicked
	 */
	void OnListViewMouseButtonDoubleClick(const TSharedPtr<FCompElementViewModel> Item)
	{
	}

	/** Handler for when an item has scrolled into view after having been requested to do so. */
	void OnItemScrolledIntoView(TSharedPtr<FCompElementViewModel> ElementItem, const TSharedPtr<ITableRow>& Widget)
	{
		// Check to see if the element wants to rename before requesting the rename.
		if(ElementItem == RequestedRenameElement.Pin())
		{
			ElementItem->BroadcastRenameRequest();
			RequestedRenameElement.Reset();
		}
	}

	/** Called when a specific row is dragged - creates a special drag/drop op. */
	FReply OnDragRow(const FGeometry& /*MyGeometry*/, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && ViewModel->GetSelectedElements().Num() > 0)
		{
			TSharedRef<FCompElementDragDropOp> DragDropOp = MakeShared<FCompElementDragDropOp>();

			for (TSharedPtr<FCompElementViewModel> Element : ViewModel->GetSelectedElements())
			{
				FName ElementName = Element->GetFName();
				if (ElementName != NAME_None)
				{
					DragDropOp->Elements.Add(ElementName);
				}

				DragDropOp->Actors.Add(Element->GetDataSource());
			}

			DragDropOp->Construct();
			return FReply::Handled().BeginDragDrop(DragDropOp);
		}

		return FReply::Unhandled();
	}

private:
	/**	Whether the view is currently updating the view model's selection */
	bool bUpdatingSelection;

	/** The UI logic of the elements view that is not slate specific */
	TSharedPtr<FCompElementCollectionViewModel> ViewModel;

	/** Our tree view widget, used to list the comp elements */
	TSharedPtr<SCompElementTreeView> TreeView;

	/** The string to highlight on any text contained in the view widget */
	TAttribute<FText> HighlightText;

	/** Used to defer a rename on a element until after it has been scrolled into view */
	TWeakPtr<FCompElementViewModel> RequestedRenameElement;
};

#undef LOCTEXT_NAMESPACE
