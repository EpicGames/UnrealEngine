// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Images/SImage.h"

class SCompPreviewImage : public SImage
{
public:
	SLATE_BEGIN_ARGS(SCompPreviewImage)
		: _Image(FCoreStyle::Get().GetDefaultBrush())
		, _ColorAndOpacity(FLinearColor::White)
		, _DrawEffects(ESlateDrawEffect::None)
		{}

		SLATE_ATTRIBUTE(const FSlateBrush*, Image)
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)
		SLATE_ATTRIBUTE(ESlateDrawEffect, DrawEffects)
	SLATE_END_ARGS()

	SCompPreviewImage()
		: DrawEffects(ESlateDrawEffect::None)
	{}

	void Construct(const FArguments& InArgs)
	{
		DrawEffects = InArgs._DrawEffects;

		SImage::Construct(
			SImage::FArguments()
				.Image(InArgs._Image)
				.ColorAndOpacity(InArgs._ColorAndOpacity)
		);
	}

	//~ Begin SWidget interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		const FSlateBrush* ImageBrush = GetImageAttribute().Get();

		if ((ImageBrush != nullptr) && (ImageBrush->DrawAs != ESlateBrushDrawType::NoDrawType))
		{
			ESlateDrawEffect PaintEffects = DrawEffects.Get();
			if (!ShouldBeEnabled(bParentEnabled))
			{
				PaintEffects |= ESlateDrawEffect::DisabledEffect;
			}

			const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint() * GetColorAndOpacityAttribute().Get().GetColor(InWidgetStyle) * ImageBrush->GetTint(InWidgetStyle));

			FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), ImageBrush, PaintEffects, FinalColorAndOpacity);
		}
		return LayerId;
	}
	//~ End SWidget interface

private:
	TAttribute<ESlateDrawEffect> DrawEffects;
};
