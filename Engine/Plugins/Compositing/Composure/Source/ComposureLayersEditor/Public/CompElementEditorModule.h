// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Framework/Commands/UICommandList.h" // for FUICommandList
#include "Framework/MultiBox/MultiBoxExtender.h" // for FExtender
#include "Delegates/DelegateCombinations.h" // for DECLARE_DELEGATE_RetVal_OneParam()

class ICompElementManager;


class ICompElementEditorModule : public IModuleInterface
{
public:
	static ICompElementEditorModule& Get();

	/** 
	 * Returns a manager object for compositing elements, which can be used to perform
	 * various operations on the element objects in the editor.
	 */
	virtual TSharedPtr<ICompElementManager> GetCompElementManager() = 0;

	/** Delegates to be called to extend the comp-element editor menus */
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<FExtender>, FCompEditorMenuExtender, const TSharedRef<FUICommandList>);
	virtual TArray<FCompEditorMenuExtender>& GetEditorMenuExtendersList() = 0;
};

