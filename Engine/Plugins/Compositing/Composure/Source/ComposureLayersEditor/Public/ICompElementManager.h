// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"

class ACompositingElement;
class UCompositingMediaCaptureOutput;
class AActor;
class UPackage;
template<typename TItemType> class IFilter;

/** */
enum class ECompElementEdActions
{
	Add,
	Modify,
	Delete,
	Rename,
	Reset
};

/** 
 * Interface for running editor operations/logic on compositing element actors.
 */
class ICompElementManager 
{
public:
 	typedef IFilter<const TWeakObjectPtr<AActor>&> FActorFilter;
 
	/**
	 * Creates a actor object for the named element.
	 *
	 * @param  ElementName	The name of the element to create.
	 * @param  ClassType	The element type to spawn.
	 * @param  LevelContext Optional actor whose world level you want to spawn into.
	 * @return The newly created actor object for the named element.
	 */
	virtual TWeakObjectPtr<ACompositingElement> CreateElement(const FName& ElementName, TSubclassOf<ACompositingElement> ClassType, AActor* LevelContext = nullptr, EObjectFlags ObjectFlags = EObjectFlags::RF_NoFlags) = 0;
 
	/**
	 * Gets the actor object of the named element.
	 *
	 * @param  ElementName	The name of the element you want to lookup
	 * @return The object Object of the provided element name
	 */
	virtual TWeakObjectPtr<ACompositingElement> GetElement(const FName& ElementName) const = 0;

	/**
	 * Attempts to get the actor object of the provided element name. 
	 *
	 * @param  ElementName		The name of the element you want to lookup
	 * @param  OutElement[OUT]	Set to the actor object of the named element. Set to Invalid if no actor object exists.
	 * @return True if a valid actor object was found and set to OutElement; otherwise false, a valid actor object was not found.
	 */
	virtual bool TryGetElement(const FName& ElementName, TWeakObjectPtr<ACompositingElement>& OutElement) = 0;

	/**
	 * Gets all known elements and appends them to the provided array.
	 *
	 * @param OutElements[OUT] Output array to store all known elements in
	 */
	virtual void AddAllCompElementsTo(TArray< TWeakObjectPtr<ACompositingElement> >& OutElements) const = 0;

	/**
	 * Deletes the specified element and all of its children.
	 *
	 * @param ElementToDelete	    A valid element name
	 * @param bIsCalledFromEditor   True if this function is called from editor. False if it is called from BP/C++. Its default value is true.
	 */
	virtual void DeleteElementAndChildren(const FName& ElementToDelete, bool bIsCalledFromEditor = true) = 0;

	/**
	 * Deletes all of the provided elements and all of their children if the function is called from editor. Use DeleteElementAndChildren instead.
	 *
	 * @param ElementsToDelete	   A list of valid element names
	 * @param bIsCalledFromEditor  True if this function is called from editor. False if it is called from BP/C++. Its default value is true.
	 */
	virtual void DeleteElements(const TArray<FName>& ElementsToDelete, bool bIsCalledFromEditor = true) = 0;

	/**
	 * Renames the element with the specified original named to the provided new name.
	 *
	 * @param	OriginalElementName The name of the element to be renamed
	 * @param	NewElementName		The new name for the element to be renamed
	 */
	virtual bool RenameElement(const FName OriginalElementName, const FName& NewElementName) = 0;

	/**
	 * Attaches the specified element as a child to the named parent.
	 *
	 * @param  ParentName	The element you want to nest the child under.
	 * @param  ElementName	A valid element name for the child you want to add.
	 * @return True if the element was successfully added as a child to the specified parent.
	 */
	virtual bool AttachCompElement(const FName ParentName, const FName ElementName) = 0;

	/**
	 * Selects/de-selects specified element actors.
	 *
	 * @param  ElementNames						A valid list of element names.
	 * @param  bSelect							If true, actors are selected; if false, actors are deselected
	 * @param  bNotify							If true, the Editor is notified of the selection change; if false, the Editor will not be notified
	 * @param  bSelectEvenIfHidden [optional]	If true, even hidden actors will be selected; if false, hidden actors won't be selected
	 * @param  Filter [optional]				Elements that don't pass the specified filter restrictions won't be selected
	 * @return 									true if at least one actor was selected/deselected.
	 */
	virtual bool SelectElementActors(const TArray<FName>& ElementNames, bool bSelect, bool bNotify, bool bSelectEvenIfHidden = false, const TSharedPtr<FActorFilter>& Filter = TSharedPtr<FActorFilter>(nullptr)) = 0;

	/**
	 * Toggles the named element's enabled state.
	 *
	 * @param ElementName	The name of the element to affect
	 */
	virtual void ToggleElementRendering(const FName& ElementName) = 0;

	/**
	 * Toggles the named element's pause state.
	 *
	 * @param ElementName	The name of the element to affect
	 */
	virtual void ToggleElementFreezeFrame(const FName& ElementName) = 0;
	
	/**
	 * Adds/Enables/Disables a media-capture output pass for the specified element.
	 * Adds & enables a media-capture pass if one doesn't already exist. Just enables/disables
	 * if one already exits.
	 *
	 * @param ElementName	The name of the element to affect
	 */
	virtual void ToggleMediaCapture(const FName& ElementName) = 0;

	/** 
	 * Re-Prompts the user to select a UMediaOutput asset to associate the name element's 
	 * media-capture pass with.
	 *
	 * @param ElementName	The name of the element to affect
	 */
	virtual UCompositingMediaCaptureOutput* ResetMediaCapture(const FName& ElementName) = 0;

	/** 
	 * Deletes any media-capture passes from the specified element.
	 *
	 * @param ElementName	The name of the element to affect
	 */
	virtual void RemoveMediaCapture(const FName& ElementName) = 0;

	/** 
	 * Re-queries the scene for element actors and rebuilds the authoritative list used by the editor.
	 */
	virtual void RefreshElementsList() = 0;

	/** 
	 * Dirties the draw state so editor element instance get ran/rendered this frame (utilized 
	 * when rendering on demand, instead of intrinsically each frame).
	 */
	virtual void RequestRedraw() = 0;

	/** 
	 * Determines if the specified element is being rendered by the hidden compositing viewport.
	 * 
	 * @param  CompElement	The element actor you're querying for
	 * @return True if the game-thread is in the middle of queuing the specified element.
	 */
	virtual bool IsDrawing(ACompositingElement* CompElement) const = 0;

	/**
	 * Function used to expose the editor call back function after a new composure element is created via Blueprint and C++.
	 * 
	 * @param  NewElement	                             The new element that got created via Blueprint/C++
	 */
	virtual void OnCreateNewElement(AActor* NewElement) = 0;

	/**
	 * Function used to expose the editor call back function after an existing composure element 
	 *
	 * @param  ElementToDelete	                         The composure element that is going to be deleted.
	 */
	virtual void OnDeleteElement(AActor* ElementToDelete) = 0;

	
	/** Broadcasts whenever one or more elements are modified */
	DECLARE_EVENT_ThreeParams(ICompElementManager, FOnElementsChanged, const ECompElementEdActions /*Action*/, const TWeakObjectPtr<ACompositingElement>& /*ElementObj*/, const FName& /*ChangedProperty*/);
	virtual FOnElementsChanged& OnElementsChanged() = 0;
};
